{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NNEDenoiser",
	"Description": "Neural denoiser for the Unreal Path Tracer based on the Neural Network Engine (NNE).",
	"Category": "Denoising",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault" : true,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64", "Linux", "Mac"
	],
	"Modules": [
		{
			"Name": "NNEDenoiser",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NNEDenoiserShaders",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "NNERuntimeORT",
			"Enabled": true
		}
	]
}