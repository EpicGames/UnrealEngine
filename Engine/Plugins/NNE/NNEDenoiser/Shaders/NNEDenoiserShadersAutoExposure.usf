// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

int InputTextureWidth;
int InputTextureHeight;
Texture2D InputTexture;
RWBuffer<float> OutputBuffer;

[numthreads(1, 1, 1)]
void AutoExposure(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const float Key = 0.18;

	float Sum = 0.0;
	int Count = 0;
	for (int i = 0; i < InputTextureWidth; i++)
	{
		for (int j = 0; j < InputTextureHeight; j++)
		{
			float3 Color = InputTexture.Load(int3(i, j, 0)).rgb;

			Color = clamp(Color, float3(0.0, 0.0, 0.0), float3(MAX_FLT, MAX_FLT, MAX_FLT));

			float Lum = Luminance(Color);
			if (Lum > 1e-8)
			{
				Sum += log2(Lum);
				Count++;
			}
		}
	}

	float Result = Count > 0 ? Sum / (float)Count : 1.0;
	Result = Key / exp2(Result);

	OutputBuffer[0] = Result;
	OutputBuffer[1] = 1.0 / Result;
}