// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

int InputTextureWidth;
int InputTextureHeight;
Texture2D InputTexture;

int NumBinsW;
int NumBinsH;
RWBuffer<float> OutputBins;

groupshared float Sums[THREAD_GROUP_SIZE * THREAD_GROUP_SIZE];

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void Downsample(in const uint3 GroupThreadID : SV_GroupThreadID, in const uint3 GroupID : SV_GroupID, in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const int BeginW = GroupID.x * InputTextureWidth / NumBinsW;
	const int BeginH = GroupID.y * InputTextureHeight / NumBinsH;
	const int EndW = (GroupID.x + 1) * InputTextureWidth / NumBinsW;
	const int EndH = (GroupID.y + 1) * InputTextureHeight / NumBinsH;

	const int CoordW = BeginW + GroupThreadID.x;
	const int CoordH = BeginH + GroupThreadID.y;

	float Lum = 0.0;
	if (CoordW < EndW && CoordH < EndH)
	{
		float3 Color = InputTexture.Load(int3(CoordW, CoordH, 0)).rgb;
		Color = clamp(Color, float3(0.0, 0.0, 0.0), float3(MAX_FLT, MAX_FLT, MAX_FLT));

		Lum = Luminance(Color);
	}

	const int LocalID = GroupThreadID.y * THREAD_GROUP_SIZE + GroupThreadID.x;
	Sums[LocalID] = Lum;

	for (int i = THREAD_GROUP_SIZE * THREAD_GROUP_SIZE / 2; i > 0; i >>= 1)
	{
		GroupMemoryBarrierWithGroupSync();
		
		if (LocalID < i)
		{
			Sums[LocalID] += Sums[LocalID + i];
		}
	}

	if (LocalID == 0)
	{
		const float AverageLum = Sums[0] / float((EndW - BeginW) * (EndH - BeginH));
		const int GroupLinearID = GroupID.y * NumBinsW + GroupID.x;
		OutputBins[GroupLinearID] = AverageLum;
	}
}