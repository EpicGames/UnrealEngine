// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

int InputSize;
Buffer<float> InputBins;
RWBuffer<float> OutputSums;
RWBuffer<int> OutputCounts;
int NumThreads;

groupshared float Sums[THREAD_GROUP_SIZE];
groupshared int Counts[THREAD_GROUP_SIZE];

[numthreads(THREAD_GROUP_SIZE, 1, 1)]
void Reduce(in const uint3 GroupThreadID : SV_GroupThreadID, in const uint3 GroupID : SV_GroupID, in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	float LocalSum = 0.0;
	int LocalCount = 0;
	for (int i = DispatchThreadID.x; i < InputSize; i += NumThreads)
	{
		const float Lum = InputBins[i];
		if (Lum > EPS)
		{
			LocalSum += log2(Lum);
			++LocalCount;
		}
	}

	const int LocalID = GroupThreadID.x;
	Sums[LocalID] = LocalSum;
	Counts[LocalID] = LocalCount;

	for (int i = THREAD_GROUP_SIZE / 2; i > 0; i >>= 1)
	{
		GroupMemoryBarrierWithGroupSync();

		if (LocalID < i)
		{
			Sums[LocalID] += Sums[LocalID + i];
			Counts[LocalID] += Counts[LocalID + i];
		}
	}

	if (LocalID == 0)
	{
		OutputSums[GroupID.x] = Sums[0];
		OutputCounts[GroupID.x] = Counts[0];
	}
}