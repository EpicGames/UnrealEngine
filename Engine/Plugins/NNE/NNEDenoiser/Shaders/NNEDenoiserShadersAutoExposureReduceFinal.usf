// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

int InputSize;
Buffer<float> InputSums;
Buffer<int> InputCounts;
RWBuffer<float> OutputBuffer; // 0: Result, 1: 1.0 / Result

groupshared float Sums[THREAD_GROUP_SIZE];
groupshared int Counts[THREAD_GROUP_SIZE];

[numthreads(THREAD_GROUP_SIZE, 1, 1)]
void ReduceFinal(in const uint3 GroupThreadID : SV_GroupThreadID)
{
	const int LocalID = GroupThreadID.x;

	if (LocalID < InputSize)
	{
		Sums[LocalID] = InputSums[LocalID];
		Counts[LocalID] = InputCounts[LocalID];
	}
	else
	{
		Sums[LocalID] = 0;
		Counts[LocalID] = 0;
	}

	for (int i = THREAD_GROUP_SIZE / 2; i > 0; i >>= 1)
	{
		GroupMemoryBarrierWithGroupSync();

		if (LocalID < i)
		{
			Sums[LocalID] += Sums[LocalID + i];
			Counts[LocalID] += Counts[LocalID + i];
		}
	}

	if (LocalID == 0)
	{
		OutputBuffer[0] = (Counts[0] > 0) ? (KEY / exp2(Sums[0] / float(Counts[0]))) : 1.0;
		OutputBuffer[1] = 1.0 / OutputBuffer[0];
	}
}