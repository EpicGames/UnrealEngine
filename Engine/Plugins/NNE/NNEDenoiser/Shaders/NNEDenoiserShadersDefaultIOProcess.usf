// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

int Width;
int Height;
Texture2D InputTexture;
RWTexture2D<float4> OutputTexture;

float3 GetProcessedValue(float3 Value)
{
#if INPUT_KIND_INDEX == 0 // Input color
	return clamp(Value, float3(0.0, 0.0, 0.0), float3(MAX_FLT, MAX_FLT, MAX_FLT));
#elif INPUT_KIND_INDEX == 1 // Input albedo
	return clamp(Value, float3(0.0, 0.0, 0.0), float3(1.0, 1.0, 1.0));
#elif INPUT_KIND_INDEX == 2 // Input normal
	return clamp(Value, float3(-1.0, -1.0, -1.0), float3(1.0, 1.0, 1.0)) * 0.5 + float3(0.5, 0.5, 0.5);
#elif INPUT_KIND_INDEX == 3 // Input flow
	// TODO flow texture check for any necessary conversion
	return Value;
#else
	// INPUT_KIND_INDEX == 4 // Output color
	return clamp(Value, float3(0.0, 0.0, 0.0), float3(MAX_FLT, MAX_FLT, MAX_FLT));
#endif
}


[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void IOProcess(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (DispatchThreadID.x >= Width || DispatchThreadID.y >= Height)
	{
		return;
	}

	const float4 Color = InputTexture.Load(int3(DispatchThreadID.x, DispatchThreadID.y, 0));

	OutputTexture[DispatchThreadID.xy] = float4(GetProcessedValue(Color.rgb), Color.a);
}