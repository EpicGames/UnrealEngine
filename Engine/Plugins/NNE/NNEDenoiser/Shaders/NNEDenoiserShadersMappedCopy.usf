// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

#define FLOAT16 half
#define FLOAT16_4 half4

#define FLOAT32 float
#define FLOAT32_4 float4

#if INPUT_DATA_TYPE_INDEX == 4 // Float16
#define INPUT_SCALAR_DATA_TYPE FLOAT16
#define INPUT_VECTOR_DATA_TYPE FLOAT16_4
#else // INPUT_DATA_TYPE_INDEX == 3 // Float32
#define INPUT_SCALAR_DATA_TYPE FLOAT32
#define INPUT_VECTOR_DATA_TYPE FLOAT32_4
#endif

#if OUTPUT_DATA_TYPE_INDEX == 4 // Float16
#define OUTPUT_SCALAR_DATA_TYPE FLOAT16
#define OUTPUT_VECTOR_DATA_TYPE FLOAT16_4
#else // OUTPUT_DATA_TYPE_INDEX == 3 // Float32
#define OUTPUT_SCALAR_DATA_TYPE FLOAT32
#define OUTPUT_VECTOR_DATA_TYPE FLOAT32_4
#endif

#if INPUT_DATA_TYPE_INDEX == 4 // Float16
#if OUTPUT_DATA_TYPE_INDEX == 4 // Float16
#define CONVERT(X) X
#else // OUTPUT_DATA_TYPE_INDEX == 3 // Float32
#define CONVERT(X) FLOAT32(X)
#endif
#else // INPUT_DATA_TYPE_INDEX == 3 // Float32
#if OUTPUT_DATA_TYPE_INDEX == 4 // Float16
#define CONVERT(X) FLOAT16(X)
#else // OUTPUT_DATA_TYPE_INDEX == 3 // Float32
#define CONVERT(X) X
#endif
#endif

int Width;
int Height;
int4 OutputChannel_InputChannel_Unused_Unused[MAX_NUM_MAPPED_CHANNELS];

#if INTRINSIC_INPUT_TYPE == 0 // Texture
Texture2D<INPUT_VECTOR_DATA_TYPE> InputTexture;
#else // INTRINSIC_INPUT_TYPE == 1 // Buffer
Buffer<INPUT_SCALAR_DATA_TYPE> InputBuffer;
#endif

#if INTRINSIC_OUTPUT_TYPE == 0 // Texture
RWTexture2D<OUTPUT_VECTOR_DATA_TYPE> OutputTexture;
#else // INTRINSIC_OUTPUT_TYPE == 1 // Buffer
RWBuffer<OUTPUT_SCALAR_DATA_TYPE> OutputBuffer;
#endif

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void MappedCopy(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (DispatchThreadID.x >= Width || DispatchThreadID.y >= Height)
	{
		return;
	}

#if INTRINSIC_INPUT_TYPE == 1 || INTRINSIC_OUTPUT_TYPE == 1
	const int Idx = DispatchThreadID.y * Width + DispatchThreadID.x;
	const int Offset = Width * Height;
#endif

#if INTRINSIC_INPUT_TYPE == 0
	const INPUT_VECTOR_DATA_TYPE InputColor = InputTexture.Load(int3(DispatchThreadID.x, DispatchThreadID.y, 0));
#endif

#if INTRINSIC_OUTPUT_TYPE == 0
	OUTPUT_VECTOR_DATA_TYPE Result = OutputTexture[DispatchThreadID.xy];
#endif

	UNROLL
	for (int I = 0; I < NUM_MAPPED_CHANNELS; I++)
	{
		const int4 Info = OutputChannel_InputChannel_Unused_Unused[I];
		const int OutputChannel = Info.x;
		const int InputChannel = Info.y;

#if INTRINSIC_INPUT_TYPE == 0
		const INPUT_SCALAR_DATA_TYPE InputValue = InputColor[InputChannel];
#else
		const INPUT_SCALAR_DATA_TYPE InputValue = InputBuffer[Idx + InputChannel * Offset];
#endif

#if INTRINSIC_OUTPUT_TYPE == 0
		Result[OutputChannel] = CONVERT(InputValue);
#else
		OutputBuffer[Idx + OutputChannel * Offset] = CONVERT(InputValue);
#endif
	}

#if INTRINSIC_OUTPUT_TYPE == 0
	OutputTexture[DispatchThreadID.xy] = Result;
#endif
}