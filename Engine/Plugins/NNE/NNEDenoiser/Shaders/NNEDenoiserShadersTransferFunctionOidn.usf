// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

int Width;
int Height;
Texture2D InputTexture;
Buffer<float> InputScaleBuffer;
float NormScale;
float InvNormScale;
RWTexture2D<float4> OutputTexture;


///

// from https://github.com/OpenImageDenoise/oidn/blob/master/core/color.h
// struct FPU
// {
static float a  =  1.41283765e+03;
static float b  =  1.64593172e+00;
static float c  =  4.31384981e-01;
static float d  = -2.94139609e-03;
static float e  =  1.92653254e-01;
static float f  =  6.26026094e-03;
static float g  =  9.98620152e-01;
static float y0 =  1.57945760e-06;
static float y1 =  3.22087631e-02;
static float x0 =  2.23151711e-03;
static float x1 =  3.70974749e-01;

static float Forward(float y)
{
	if (y <= y0) return a * y;
	else if (y <= y1) return b * pow(y, c) + d;
	return e * log(y + f) + g;
}

static float Inverse(float x)
{
	if (x <= x0) return x / a;
	else if (x <= x1) return pow((x - d) / b, 1.0 / c);
	return exp((x - g) / e) - f;
}

float3 GetProcessedValue(float3 Value)
{
#if MODE == 0 // Forward
	Value = Value * InputScaleBuffer[0];
	Value = clamp(Value, float3(0.0, 0.0, 0.0), float3(MAX_FLT, MAX_FLT, MAX_FLT));

	return float3(Forward(Value.r), Forward(Value.g), Forward(Value.b)) * NormScale;
#else
	// MODE == 1 // Inverse
	Value = clamp(Value, float3(0.0, 0.0, 0.0), float3(MAX_FLT, MAX_FLT, MAX_FLT)) * InvNormScale;
	
	return float3(Inverse(Value.r), Inverse(Value.g), Inverse(Value.b)) * InputScaleBuffer[1];
#endif
}

// };

///

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void PreOrPostprocess(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (DispatchThreadID.x >= Width || DispatchThreadID.y >= Height)
	{
		return;
	}

	const float4 Color = InputTexture.Load(int3(DispatchThreadID.x, DispatchThreadID.y, 0));

	OutputTexture[DispatchThreadID.xy] = float4(GetProcessedValue(Color.rgb), Color.a);
}