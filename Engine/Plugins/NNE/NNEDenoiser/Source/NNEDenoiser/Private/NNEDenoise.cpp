// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiser.h"
#include "Misc/CoreDelegates.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserViewExtension.h"
#include "RenderUtils.h"

DEFINE_LOG_CATEGORY(LogNNEDenoiser);

void FNNEDenoiserModule::StartupModule()
{
#if RHI_RAYTRACING
	// During cook, we are not allowed to call IsRayTracingAllowed() later, therefore we assume Ray Tracing is not available and do not register the View Extension.
	if (IsRunningCookCommandlet())
	{
		return;
	}

	FCoreDelegates::GetOnPostEngineInit().AddLambda([this] ()
	{
		// Only register View Extension if Ray Tracing is available.
		if (IsRayTracingAllowed())
		{
			ViewExtension = FSceneViewExtensions::NewExtension<UE::NNEDenoiser::Private::FViewExtension>();
		}
		else
		{
			UE_LOGF(LogNNEDenoiser, Log, "Ray Tracing is not enabled, therefore NNEDenoiser is not registered!");
		}
	});
#endif // RHI_RAYTRACING
}

void FNNEDenoiserModule::ShutdownModule()
{
	ViewExtension.Reset();
}

IMPLEMENT_MODULE(FNNEDenoiserModule, NNEDenoiser)