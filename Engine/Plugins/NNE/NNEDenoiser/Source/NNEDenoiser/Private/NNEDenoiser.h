// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

namespace UE::NNEDenoiser::Private
{
	class FViewExtension;
}

class FNNEDenoiserModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedPtr<UE::NNEDenoiser::Private::FViewExtension, ESPMode::ThreadSafe> ViewExtension;
};
