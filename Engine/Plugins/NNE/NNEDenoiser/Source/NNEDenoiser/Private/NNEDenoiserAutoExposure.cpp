// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserAutoExposure.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserShadersAutoExposureCS.h"
#include "NNEDenoiserUtils.h"
#include "RenderGraphBuilder.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_GPU_STAT_NAMED(FAutoExposure, TEXT("AutoExposure"));
DECLARE_GPU_STAT_NAMED(FAutoExposureDownsample, TEXT("AutoExposure.Downsample"));
DECLARE_GPU_STAT_NAMED(FAutoExposureReduce, TEXT("AutoExposure.Reduce"));
DECLARE_GPU_STAT_NAMED(FAutoExposureReduceFinal, TEXT("AutoExposure.ReduceFinal"));

BEGIN_SHADER_PARAMETER_STRUCT(FAutoExposureTestDownloadParameters, )
	RDG_BUFFER_ACCESS(OutputBuffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

namespace UE::NNEDenoiser::Private
{

void FAutoExposure::Run(TConstArrayView<FLinearColor> InputData, FIntPoint Size, float& OutputValue) const
{
	using namespace UE::NNEDenoiserShaders::Internal;

	OutputValue = 0.0f;
	if (InputData.IsEmpty())
	{
		return;
	}

	const int32 NumBinsW = FMath::DivideAndRoundUp(Size.X, FAutoExposureDownsampleConstants::MAX_BIN_SIZE);
	const int32 NumBinsH = FMath::DivideAndRoundUp(Size.Y, FAutoExposureDownsampleConstants::MAX_BIN_SIZE);
	const int32 NumBins = NumBinsW * NumBinsH;

	TArray<float> Bins;
	Bins.SetNumUninitialized(NumBins);

	for (int32 GroupX = 0; GroupX < NumBinsW; GroupX++)
	{
		for (int32 GroupY = 0; GroupY < NumBinsH; GroupY++)
		{
			const int32 BeginW = GroupX * Size.X / NumBinsW;
			const int32 BeginH = GroupY * Size.Y / NumBinsH;
			const int32 EndW = (GroupX + 1) * Size.X / NumBinsW;
			const int32 EndH = (GroupY + 1) * Size.Y / NumBinsH;

			float Lum = 0.0f;
			for (int32 i = 0; i < FAutoExposureDownsampleConstants::MAX_BIN_SIZE; i++)
			{
				for (int32 j = 0; j < FAutoExposureDownsampleConstants::MAX_BIN_SIZE; j++)
				{
					const int32 CoordW = BeginW + i;
					const int32 CoordH = BeginH + j;

					if (CoordW < EndW && CoordH < EndH)
					{
						FLinearColor Color = InputData[CoordH * Size.X + CoordW];
						
						Lum += 0.212671f * Color.R + 0.715160f * Color.G + 0.072169f * Color.B;
					}
				}
			}

			Bins[GroupY * NumBinsW + GroupX] = Lum / float((EndW - BeginW) * (EndH - BeginH));
		}
	}

	float Sum = 0.0;
	int32 Count = 0;
	for (int32 i = 0; i < NumBins; i++)
	{
		const float Lum = Bins[i];
		if (Lum > FAutoExposureReduceConstants::EPS)
		{
			Sum += FMath::Log2(Lum);
			Count++;
		}
	}

	OutputValue = Count > 0 ? FAutoExposureReduceConstants::KEY / FMath::Exp2(Sum / (float)Count) : 1.0f;
}

void FAutoExposure::EnqueueRDG(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGBufferRef OutputBuffer) const
{
	using namespace UE::NNEDenoiserShaders::Internal;

	const FIntVector InputTextureSize = InputTexture->Desc.GetSize();

	const int32 NumBinsW = FMath::DivideAndRoundUp(InputTextureSize.X, FAutoExposureDownsampleConstants::MAX_BIN_SIZE);
	const int32 NumBinsH = FMath::DivideAndRoundUp(InputTextureSize.Y, FAutoExposureDownsampleConstants::MAX_BIN_SIZE);
	const int32 NumBins = NumBinsW * NumBinsH;

	const int NumReduceGroups = FMath::Min(CeilDiv(NumBins, FAutoExposureReduceConstants::THREAD_GROUP_SIZE), FAutoExposureReduceConstants::THREAD_GROUP_SIZE);

	FRDGBufferDesc OutputBinsDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), NumBins);
	FRDGBufferRef OutputBins = GraphBuilder.CreateBuffer(OutputBinsDesc, TEXT("AutoExposureOutputBins"));

	FRDGBufferDesc OutputSumsDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), NumReduceGroups);
	FRDGBufferRef OutputSums = GraphBuilder.CreateBuffer(OutputSumsDesc, TEXT("AutoExposureOutputSums"));

	FRDGBufferDesc OutputCountsDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumReduceGroups);
	FRDGBufferRef OutputCounts = GraphBuilder.CreateBuffer(OutputCountsDesc, TEXT("AutoExposureOutputCounts"));

	{
		const FIntVector GroupCount = FIntVector(NumBinsW, NumBinsH, 1);

		FAutoExposureDownsampleCS::FParameters *ShaderParameters = GraphBuilder.AllocParameters<FAutoExposureDownsampleCS::FParameters>();
		ShaderParameters->InputTextureWidth = InputTextureSize.X;
		ShaderParameters->InputTextureHeight = InputTextureSize.Y;
		ShaderParameters->InputTexture = InputTexture;
		ShaderParameters->NumBinsW = NumBinsW;
		ShaderParameters->NumBinsH = NumBinsH;
		ShaderParameters->OutputBins = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutputBins, EPixelFormat::PF_R32_FLOAT));

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FAutoExposureDownsampleCS> Shader(GlobalShaderMap);

		RDG_EVENT_SCOPE_STAT(GraphBuilder, FAutoExposureDownsample, "AutoExposure.Downsample");

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("AutoExposure.Downsample"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			Shader,
			ShaderParameters,
			GroupCount);
	}

	{
		FAutoExposureReduceCS::FParameters *ShaderParameters = GraphBuilder.AllocParameters<FAutoExposureReduceCS::FParameters>();
		ShaderParameters->InputSize = NumBins;
		ShaderParameters->InputBins = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(OutputBins, EPixelFormat::PF_R32_FLOAT));
		ShaderParameters->OutputSums = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutputSums, EPixelFormat::PF_R32_FLOAT));
		ShaderParameters->OutputCounts = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutputCounts, EPixelFormat::PF_R32_SINT));
		ShaderParameters->NumThreads = NumReduceGroups * FAutoExposureDownsampleConstants::MAX_BIN_SIZE;

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FAutoExposureReduceCS> Shader(GlobalShaderMap);

		RDG_EVENT_SCOPE_STAT(GraphBuilder, FAutoExposureReduce, "AutoExposure.Recude");

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("AutoExposure.Recude"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			Shader,
			ShaderParameters,
			FIntVector(NumReduceGroups, 1, 1));
	}

	{
		FAutoExposureReduceFinalCS::FParameters *ShaderParameters = GraphBuilder.AllocParameters<FAutoExposureReduceFinalCS::FParameters>();
		ShaderParameters->InputSize = NumReduceGroups;
		ShaderParameters->InputSums = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(OutputSums, EPixelFormat::PF_R32_FLOAT));
		ShaderParameters->InputCounts = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(OutputCounts, EPixelFormat::PF_R32_SINT));
		ShaderParameters->OutputBuffer = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutputBuffer, EPixelFormat::PF_R32_FLOAT));

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FAutoExposureReduceFinalCS> Shader(GlobalShaderMap);

		RDG_EVENT_SCOPE_STAT(GraphBuilder, FAutoExposureReduceFinal, "AutoExposure.RecudeFinal");

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("AutoExposure.RecudeFinal"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			Shader,
			ShaderParameters,
			FIntVector(1, 1, 1));
	}
}

}