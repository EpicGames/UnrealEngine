// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Math/MathFwd.h"
#include "RenderGraphFwd.h"

struct FLinearColor;

namespace UE::NNEDenoiser::Private
{

class IAutoExposure
{
public:
	virtual ~IAutoExposure() = default;

	virtual void Run(TConstArrayView<FLinearColor> InputData, FIntPoint Size, float& OutputValue) const = 0;
	virtual void EnqueueRDG(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGBufferRef OutputBuffer) const = 0;
};

class FAutoExposure : public IAutoExposure
{
public:
	virtual ~FAutoExposure() = default;

	virtual void Run(TConstArrayView<FLinearColor> InputData, FIntPoint Size, float& OutputValue) const override;
	virtual void EnqueueRDG(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGBufferRef OutputBuffer) const override;
};

}