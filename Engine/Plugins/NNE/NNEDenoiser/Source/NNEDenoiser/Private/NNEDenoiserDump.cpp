// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserDump.h"

#ifdef WITH_NNEDENOISER_DUMP
#include "HAL/PlatformFileManager.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogVerbosity.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "NNEDenoiserLog.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphEvent.h"
#include "RenderingThread.h"
#include "RHIGPUReadback.h"
#include "Tasks/Task.h"
#include "Windows/WindowsPlatformProcess.h"

BEGIN_SHADER_PARAMETER_STRUCT(FDumpBufferPassParameters, )
	RDG_BUFFER_ACCESS(Buffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

namespace UE::NNEDenoiser::Private
{

FString GetDefaultDumpPath()
{
	const FDateTime NowUTC = FDateTime::UtcNow();
	const FString Stamp = NowUTC.ToString(TEXT("%Y-%m-%d_%H-%M-%S"));
	FString Base = FString::Printf(TEXT("NNEDenoiserDump_%s"), *Stamp);
	const FString DumpFilePath = FPaths::ProjectSavedDir() / TEXT("NNEDenoiserDumps") / FPaths::MakeValidFileName(Base);
	
	return DumpFilePath;
}

void ScheduleDumpBuffer(FRDGBuilder& GraphBuilder, FRDGBufferRef Buffer, const FString& Name, const FString& FilePath)
{
	FString FilePathLocal = FilePath;

	static FString DefaultPath;
	if (FilePathLocal.IsEmpty())
	{
		if (DefaultPath.IsEmpty())
		{
			DefaultPath = GetDefaultDumpPath();
		}
		FilePathLocal = DefaultPath;
	}
	else
	{
		DefaultPath = FilePathLocal;
	}

	FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(*FilePathLocal);

	TSharedRef<FRHIGPUBufferReadback> Readback = MakeShared<FRHIGPUBufferReadback>(FName(Name));
	FRHIGPUBufferReadback* ReadbackPtr = &Readback.Get();

	auto* PassParameters = GraphBuilder.AllocParameters<FDumpBufferPassParameters>();
	PassParameters->Buffer = Buffer;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("DebugDumpBuffer(%s)", *Name),
		PassParameters,
		ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
		[PassParameters, ReadbackPtr](FRHICommandList& RHICommandList)
		{
			ReadbackPtr->EnqueueCopy(RHICommandList, PassParameters->Buffer->GetRHI());
		}
	);

	Tasks::Launch(UE_SOURCE_LOCATION, [Readback, Name, FilePathLocal]()
	{
		FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool();
		TArray<uint8> FileData;

		while (!Readback->IsReady())
		{
			FPlatformProcess::Sleep(0.01f);
		}

		ENQUEUE_RENDER_COMMAND(ReadRHIBuffer)([Readback, Name, Signal, &FileData](FRHICommandListImmediate& RHICmdList)
		{	
			int32 Size = Readback->GetGPUSizeBytes();
			if (Size > 0)
			{
				void* Data = Readback->Lock(Size);
				if (Data)
				{
					FileData.SetNumUninitialized(Size);
					FMemory::Memcpy(FileData.GetData(), Data, Size);
				}
				Readback->Unlock();
			}
			
			Signal->Trigger();
		});

		Signal->Wait();

		FGenericPlatformProcess::ReturnSynchEventToPool(Signal);
		Signal = nullptr;

		if (FileData.IsEmpty())
		{
			UE_LOGF(LogNNEDenoiser, Warning, "No data written for: %ls (empty readback)", *Name);
			return;
		}

		FString Filename = (!FilePathLocal.IsEmpty() ? FilePathLocal : FPaths::ProjectSavedDir()) / Name + TEXT(".bin");
		if (FFileHelper::SaveArrayToFile(FileData, *Filename))
		{
			UE_LOGF(LogNNEDenoiser, Log, "Dumped buffer '%ls' to %ls (%d bytes)", *Name, *Filename, FileData.Num());
		}
		else
		{
			UE_LOGF(LogNNEDenoiser, Error, "Failed to write dump for '%ls' to %ls", *Name, *Filename);
		}
	});
}

} // namespace UE::NNEDenoiser::Private

#endif // WITH_NNEDENOISER_DUMP