// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNEDENOISER_DUMP

#include "RenderGraphFwd.h"
#include "Containers/UnrealString.h"

namespace UE::NNEDenoiser::Private
{

FString GetDefaultDumpPath();

void ScheduleDumpBuffer(FRDGBuilder& GraphBuilder, FRDGBufferRef Buffer, const FString& Name, const FString& FilePath = FString());

} // namespace UE::NNEDenoiser::Private

#define NNEDENOISER_DUMP_BUFFERS_SCHEDULE(GraphBuilder, Buffers, Prefix) \
	for (int32 i = 0; i < Buffers.Num(); i++) \
	{ \
		UE::NNEDenoiser::Private::ScheduleDumpBuffer(GraphBuilder, Buffers[i], FString::Printf(TEXT("%s_%d"), Prefix, i)); \
	}
#else
#define NNEDENOISER_DUMP_BUFFERS_SCHEDULE(GraphBuilder, Buffers, Prefix)
#endif // WITH_NNEDENOISER_DUMP