// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserGenericDenoiser.h"
#include "Algo/Transform.h"
#include "NNEDenoiserAutoExposure.h"
#include "NNEDenoiserDump.h"
#include "NNEDenoiserHistory.h"
#include "NNEDenoiserIOProcess.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserModelInstance.h"
#include "NNEDenoiserResourceManager.h"
#include "NNEDenoiserTransferFunction.h"
#include "NNEDenoiserUtils.h"
#include <type_traits>

namespace UE::NNEDenoiser::Private
{

class FResourceAccess : public IResourceAccess
{
public:
	FResourceAccess(const FResourceManager& ResourceManager) : ResourceManager(ResourceManager) { }

	virtual FRDGTextureRef GetTexture(EResourceName TensorName, int32 FrameIdx) const override
	{
		return ResourceManager.GetTexture(TensorName, FrameIdx);
	}

	virtual FRDGTextureRef GetIntermediateTexture(EResourceName TensorName, int32 FrameIdx) const override
	{
		return ResourceManager.GetIntermediateTexture(TensorName, FrameIdx);
	}

private:
	const FResourceManager& ResourceManager;
};

FRDGBufferRef CreateBufferRDG(FRDGBuilder& GraphBuilder, const NNE::FTensorDesc& TensorDesc, const NNE::FTensorShape& TensorShape)
{
	const uint32 BytesPerElement = TensorDesc.GetElementByteSize();
	const uint32 NumElements = TensorShape.Volume();

	FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(BytesPerElement, NumElements);
	Desc.Usage |= EBufferUsageFlags::NNE;

	return GraphBuilder.CreateBuffer(Desc, *TensorDesc.GetName());
}

TArray<FRDGBufferRef> CreateBuffersRDG(FRDGBuilder& GraphBuilder, TConstArrayView<NNE::FTensorDesc> TensorDescs, TConstArrayView<NNE::FTensorShape> TensorShapes)
{
	check(TensorDescs.Num() == TensorShapes.Num());

	TArray<FRDGBufferRef> Result;
	Result.SetNumUninitialized(TensorDescs.Num());

	for (int32 I = 0; I < TensorDescs.Num(); I++)
	{
		Result[I] = CreateBufferRDG(GraphBuilder, TensorDescs[I], TensorShapes[I]);
	}
	return Result;
}

TArray<NNE::FTensorBindingRDG> GetBindingRDG(TConstArrayView<FRDGBufferRef> Buffers)
{
	TArray<NNE::FTensorBindingRDG> Result;
	Result.Reserve(Buffers.Num());
	
	Algo::Transform(Buffers, Result, [] (FRDGBuffer* Buffer) { return NNE::FTensorBindingRDG{Buffer}; });

	return Result;
}

void RegisterTextureIfNeeded(FResourceManager& ResourceManager, FRDGTextureRef Texture, EResourceName ResourceName, const IInputProcess& InputProcess)
{
	const int32 MinNumFrames = ResourceName == EResourceName::Output ? 1 : 0;
	const int32 NumFrames = FMath::Max(InputProcess.NumFrames(ResourceName), MinNumFrames);
	if (NumFrames <= 0)
	{
		return;
	}
	
	ResourceManager.AddTexture(ResourceName, Texture, NumFrames);
}

TArray<NNE::FTensorShape> GetOutputTensorShapes(IModelInstance& ModelInstance)
{
	TConstArrayView<NNE::FTensorShape> InputShapes = ModelInstance.GetInputTensorShapes();
	check(!InputShapes.IsEmpty());

	TConstArrayView<uint32> InputShapeData = InputShapes[0].GetData();
	const uint32 Width = InputShapeData[3];
	const uint32 Height = InputShapeData[2];

	TConstArrayView<NNE::FTensorDesc> OutputDescs = ModelInstance.GetOutputTensorDescs();
	TConstArrayView<NNE::FTensorShape> OutputShapes = ModelInstance.GetOutputTensorShapes();

	if (OutputDescs.Num() == OutputShapes.Num())
	{
		return TArray<NNE::FTensorShape>(OutputShapes);
	}

	// If output shapes not set yet, try to manually resolve, otherwise they need to be user specified.
	TArray<NNE::FTensorShape> Result;
	for (int32 i = 0; i < OutputDescs.Num(); i++)
	{
		check(OutputDescs[i].GetShape().Rank() == 4);

		TConstArrayView<int32> SymbolicShapeData = OutputDescs[i].GetShape().GetData();
		
		TArray<uint32> ShapeData;
		ShapeData.SetNumUninitialized(OutputDescs[i].GetShape().Rank());
		ShapeData[0] = (uint32)SymbolicShapeData[0];
		ShapeData[1] = (uint32)SymbolicShapeData[1];
		ShapeData[2] = (uint32)Height;
		ShapeData[3] = (uint32)Width;

		Result.Add(NNE::FTensorShape::Make(ShapeData));
	}
	return Result;
}

void AddTilePasses(FRDGBuilder& GraphBuilder, IModelInstance& ModelInstance, const IInputProcess& InputProcess, const IOutputProcess& OutputProcess,
		FResourceManager& ResourceManager, TConstArrayView<FRDGBufferRef> InputBuffers, TConstArrayView<FRDGBufferRef> OutputBuffers)
{
	FResourceAccess ResourceAccess(ResourceManager);

	// 1. Read input and write to input buffers
	InputProcess.AddPasses(GraphBuilder, ModelInstance.GetInputTensorDescs(), ModelInstance.GetInputTensorShapes(), ResourceAccess, InputBuffers);

	NNEDENOISER_DUMP_BUFFERS_SCHEDULE(GraphBuilder, InputBuffers, TEXT("NNE_input"))
	
	// 2. Create buffer binding and infer the model
	NNE::IModelInstanceRDG::EEnqueueRDGStatus Status = ModelInstance.EnqueueRDG(GraphBuilder, GetBindingRDG(InputBuffers), GetBindingRDG(OutputBuffers));
	checkf(Status == NNE::IModelInstanceRDG::EEnqueueRDGStatus::Ok, TEXT("EnqueueRDG failed: %d"), static_cast<int>(Status));

	NNEDENOISER_DUMP_BUFFERS_SCHEDULE(GraphBuilder, OutputBuffers, TEXT("NNE_output"))

	// 3. Write output based on output buffer
	FRDGTextureRef OutputTexture = ResourceManager.GetTexture(EResourceName::Output, 0);
	OutputProcess.AddPasses(GraphBuilder, ModelInstance.GetOutputTensorDescs(), GetOutputTensorShapes(ModelInstance), ResourceAccess, OutputBuffers, OutputTexture);
}

FGenericDenoiser::FGenericDenoiser(
	TUniquePtr<IModelInstance> ModelInstance,
	TUniquePtr<IInputProcess> InputProcess,
	TUniquePtr<IOutputProcess> OutputProcess,
	FParameters DenoiserParameters,
	TUniquePtr<IAutoExposure> AutoExposure,
	TSharedPtr<ITransferFunction> TransferFunction) :
		ModelInstance(MoveTemp(ModelInstance)),
		InputProcess(MoveTemp(InputProcess)),
		OutputProcess(MoveTemp(OutputProcess)),
		DenoiserParameters(DenoiserParameters),
		AutoExposure(MoveTemp(AutoExposure)),
		TransferFunction(TransferFunction)
{

}

FGenericDenoiser::~FGenericDenoiser()
{

}

bool FGenericDenoiser::Prepare(FIntPoint Extent)
{
	if (Extent == LastExtent)
	{
		return true;
	}

	// Probably would be enough to do this only once at the very beginning...
	// We just want to be sure that everything up to width and height is correct
	if (!InputProcess->Validate(*ModelInstance, {-1, -1}))
	{
		return false;
	}

	TConstArrayView<int32> SymbolicInputShape = ModelInstance->GetInputTensorDescs()[0].GetShape().GetData();
	const FIntPoint TargetTileSize = {SymbolicInputShape[3], SymbolicInputShape[2]};

	Tiling = CreateTiling(TargetTileSize,
				DenoiserParameters.TilingConfig.MaxSize,
				DenoiserParameters.TilingConfig.MinSize,
				DenoiserParameters.TilingConfig.Alignment,
				DenoiserParameters.TilingConfig.Overlap,
				Extent);

	if (!InputProcess->Prepare(*ModelInstance, Tiling.TileSize))
	{
		return false;
	}

	LastExtent = Extent;

	UE_LOGF(LogNNEDenoiser, Log, "Prepared neural denoiser model:\n  Viewport   %dx%d\n  Num. tiles %dx%d\n  Tile size  %dx%d",
		Extent.X, Extent.Y, Tiling.Count.X, Tiling.Count.Y, Tiling.TileSize.X, Tiling.TileSize.Y);

	return true;
}

TUniquePtr<FHistory> FGenericDenoiser::AddPasses(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef ColorTex,
	FRDGTextureRef AlbedoTex,
	FRDGTextureRef NormalTex,
	FRDGTextureRef OutputTex,
	FRDGTextureRef FlowTex,
	const FRHIGPUMask& GPUMask,
	FHistory* History)
{
	using namespace UE::NNEDenoiserShaders::Internal;

	SCOPED_NAMED_EVENT_TEXT("NNEDenoiser.AddPasses", FColor::Magenta);

	// note: FlowTex is currently optional and can be null
	check(ColorTex != FRDGTextureRef{});
	check(AlbedoTex != FRDGTextureRef{});
	check(NormalTex != FRDGTextureRef{});
	check(OutputTex != FRDGTextureRef{});

	const FIntPoint Extent = ColorTex->Desc.Extent;

	TUniquePtr<FHistory> Result;

	if (!Prepare(Extent))
	{
		AddCopyTexturePass(GraphBuilder, ColorTex, OutputTex, FRHICopyTextureInfo{});

		return Result;
	}

	TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>> ResourceMap;
	if (History)
	{
		ResourceMap = History->GetResourceMap();
	}

	FResourceManager ResourceManager(GraphBuilder, Tiling, ResourceMap);
	RegisterTextureIfNeeded(ResourceManager, ColorTex, EResourceName::Color, *InputProcess);
	RegisterTextureIfNeeded(ResourceManager, AlbedoTex, EResourceName::Albedo, *InputProcess);
	RegisterTextureIfNeeded(ResourceManager, NormalTex, EResourceName::Normal, *InputProcess);
	if (FlowTex != FRDGTextureRef{})
	{
		RegisterTextureIfNeeded(ResourceManager, FlowTex, EResourceName::Flow, *InputProcess);
	}
	RegisterTextureIfNeeded(ResourceManager, OutputTex, EResourceName::Output, *InputProcess);

	TArray<FRDGBufferRef> InputBuffers = CreateBuffersRDG(GraphBuilder, ModelInstance->GetInputTensorDescs(), ModelInstance->GetInputTensorShapes());
	TArray<FRDGBufferRef> OutputBuffers = CreateBuffersRDG(GraphBuilder, ModelInstance->GetOutputTensorDescs(), GetOutputTensorShapes(*ModelInstance));

	if (TransferFunction.IsValid() && AutoExposure.IsValid())
	{
		FRDGBufferDesc InputBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), 2);
		InputBufferDesc.Usage |= EBufferUsageFlags::NNE;

		FRDGBufferRef InputScaleBuffer = GraphBuilder.CreateBuffer(InputBufferDesc, TEXT("AutoExposureOutputBuffer"));

		AutoExposure->EnqueueRDG(GraphBuilder, ColorTex, InputScaleBuffer);

		TransferFunction->RDGSetInputScale(InputScaleBuffer);
	}
	else
	{
		checkf(!TransferFunction.IsValid() && !AutoExposure.IsValid(), TEXT("TransferFunction and AutoExposure either both need to be set or not set."))
	}

	for (int32 I = 0; I < Tiling.Tiles.Num(); I++)
	{
		ResourceManager.BeginTile(I);

		// Do inference on tile
		AddTilePasses(GraphBuilder, *ModelInstance, *InputProcess, *OutputProcess, ResourceManager, InputBuffers, OutputBuffers);

		ResourceManager.EndTile();
	}

	ResourceMap = ResourceManager.MakeHistoryResourceMap();
	if (!ResourceMap.IsEmpty())
	{
		Result = MakeUnique<FHistory>(*DebugName, MoveTemp(ResourceMap));
	}

	return Result;
}

} // namespace UE::NNEDenoiser::Private