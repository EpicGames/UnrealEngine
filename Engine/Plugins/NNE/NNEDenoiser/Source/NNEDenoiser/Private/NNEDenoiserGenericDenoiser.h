// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/IntPoint.h"
#include "NNEDenoiserParameters.h"
#include "NNEDenoiserTiling.h"
#include "RenderGraphFwd.h"
#include "Templates/UniquePtr.h"

struct FRHIGPUMask;

namespace UE::NNEDenoiser::Private
{

class FHistory;
class IModelInstance;
class IInputProcess;
class IOutputProcess;
class IAutoExposure;
class ITransferFunction;

class FGenericDenoiser
{
public:
	FGenericDenoiser(
		TUniquePtr<IModelInstance> ModelInstance,
		TUniquePtr<IInputProcess> InputProcess,
		TUniquePtr<IOutputProcess> OutputProcess,
		FParameters DenoiserParameters,
		TUniquePtr<IAutoExposure> AutoExposure,
		TSharedPtr<ITransferFunction> TransferFunction
	);

	~FGenericDenoiser();

	// IPathTracingSpatialTemporalDenoiser interface
	const TCHAR* GetDebugName() const { return *DebugName; }

	TUniquePtr<FHistory> AddPasses(
		FRDGBuilder &GraphBuilder,
		FRDGTextureRef ColorTex,
		FRDGTextureRef AlbedoTex,
		FRDGTextureRef NormalTex,
		FRDGTextureRef OutputTex,
		FRDGTextureRef FlowTex,
		const FRHIGPUMask& GPUMask,
		FHistory* History);

private:
	bool Prepare(FIntPoint Extent);

	inline static const FString DebugName = TEXT("FGenericDenoiser");

	TUniquePtr<IModelInstance> ModelInstance;
	TUniquePtr<IInputProcess> InputProcess;
	TUniquePtr<IOutputProcess> OutputProcess;
	FParameters DenoiserParameters;
	TUniquePtr<IAutoExposure> AutoExposure;
	TSharedPtr<ITransferFunction> TransferFunction;

	FIntPoint LastExtent = { -1, -1 };
	FTiling Tiling{};
};

} // namespace UE::NNEDenoiser::Private