// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserResourceName.h"
#include "PathTracingDenoiser.h"
#include "RendererInterface.h"

struct IPooledRenderTarget;

namespace UE::NNEDenoiser::Private
{

using UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser;

class FHistory : public IPathTracingSpatialTemporalDenoiser::IHistory
{
public:
	FHistory(const TCHAR* DebugName, TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>> ResourceMap) : DebugName(DebugName), ResourceMap(MoveTemp(ResourceMap)) {

	}

	virtual ~FHistory() = default;

	const TCHAR* GetDebugName() const override { return DebugName; };

	const TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>>& GetResourceMap() const { return ResourceMap; }

private:
	const TCHAR* DebugName;
	TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>> ResourceMap;
};

}