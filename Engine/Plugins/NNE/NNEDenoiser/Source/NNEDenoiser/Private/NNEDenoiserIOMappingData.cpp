// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserIOMappingData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NNEDenoiserIOMappingData)

namespace UE::NNEDenoiser
{
	
EResourceName ToResourceName(EInputResourceName Name)
{
	switch(Name)
	{
		case EInputResourceName::Color: return EResourceName::Color;
		case EInputResourceName::Albedo: return EResourceName::Albedo;
		case EInputResourceName::Normal: return EResourceName::Normal;
		case EInputResourceName::Output: return EResourceName::Output;
		default: checkNoEntry();
	}
	return EResourceName::Color;
}

EResourceName ToResourceName(EOutputResourceName Name)
{
	switch(Name)
	{
		case EOutputResourceName::Output: return EResourceName::Output;
		default: checkNoEntry();
	}
	return EResourceName::Color;
}

EResourceName ToResourceName(ETemporalInputResourceName Name)
{
	switch(Name)
	{
		case ETemporalInputResourceName::Color: return EResourceName::Color;
		case ETemporalInputResourceName::Albedo: return EResourceName::Albedo;
		case ETemporalInputResourceName::Normal: return EResourceName::Normal;
		case ETemporalInputResourceName::Flow: return EResourceName::Flow;
		case ETemporalInputResourceName::Output: return EResourceName::Output;
		default: checkNoEntry();
	}
	return EResourceName::Color;
}

EResourceName ToResourceName(ETemporalOutputResourceName Name)
{
	switch(Name)
	{
		case ETemporalOutputResourceName::Output: return EResourceName::Output;
		default: checkNoEntry();
	}
	return EResourceName::Color;
}

} // namespace UE::NNEDenoiser
