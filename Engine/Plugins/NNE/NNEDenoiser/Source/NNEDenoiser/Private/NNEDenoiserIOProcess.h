// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEDenoiserResourceName.h"
#include "NNETypes.h"
#include "RenderGraphFwd.h"

namespace UE::NNEDenoiser::Private
{

class FResourceMapping;
class IModelInstance;

class IResourceAccess
{
public:
	virtual FRDGTextureRef GetTexture(EResourceName TensorName, int32 FrameIdx) const = 0;
	virtual FRDGTextureRef GetIntermediateTexture(EResourceName TensorName, int32 FrameIdx) const = 0;
};

class IInputProcess
{
public:
	virtual ~IInputProcess() = default;

	virtual bool Validate(const IModelInstance& ModelInstance, FIntPoint Extent) const = 0;

	virtual bool Prepare(IModelInstance& ModelInstance, FIntPoint Extent) const = 0;

	virtual int32 NumFrames(EResourceName Name) const = 0;

	virtual void AddPasses(
		FRDGBuilder& GraphBuilder,
		TConstArrayView<UE::NNE::FTensorDesc> TensorDescs,
		TConstArrayView<UE::NNE::FTensorShape> TensorShapes,
		const IResourceAccess& ResourceAccess,
		TConstArrayView<FRDGBufferRef> OutputBuffers) const = 0;
};

class IOutputProcess
{
public:
	virtual ~IOutputProcess() = default;

	virtual bool Validate(const IModelInstance& ModelInstance, FIntPoint Extent) const = 0;

	virtual void AddPasses(
		FRDGBuilder& GraphBuilder,
		TConstArrayView<UE::NNE::FTensorDesc> TensorDescs,
		TConstArrayView<UE::NNE::FTensorShape> TensorShapes,
		const IResourceAccess& ResourceAccess,
		TConstArrayView<FRDGBufferRef> Buffers,
		FRDGTextureRef OutputTexture) const = 0;
};

} // namespace UE::NNEDenoiser::Private