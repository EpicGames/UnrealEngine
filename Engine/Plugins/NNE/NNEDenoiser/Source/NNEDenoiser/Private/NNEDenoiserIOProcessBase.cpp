// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserIOProcessBase.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserModelInstance.h"
#include "NNEDenoiserParameters.h"
#include "NNEDenoiserShadersMappedCopyCS.h"
#include "NNEDenoiserShadersDefaultIOProcessCS.h"
#include "NNEDenoiserTransferFunction.h"
#include "NNEDenoiserUtils.h"
#include "RHIStaticStates.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_GPU_STAT_NAMED(FNNEDenoiserReadInput, TEXT("NNEDenoiser.ReadInput"));
DECLARE_GPU_STAT_NAMED(FNNEDenoiserWriteOutput, TEXT("NNEDenoiser.WriteOutput"));
DECLARE_GPU_STAT_NAMED(FNNEDenoiserDefaultIOProcess, TEXT("NNEDenoiser.DefaultIOProcess"));

namespace UE::NNEDenoiser::Private
{

namespace IOProcessBaseHelper
{

NNEDenoiserShaders::Internal::EDefaultIOProcessInputKind GetInputKind(EResourceName TensorName)
{
	using NNEDenoiserShaders::Internal::EDefaultIOProcessInputKind;
	
	switch(TensorName)
	{
		case EResourceName::Color:	return EDefaultIOProcessInputKind::Color;
		case EResourceName::Albedo:	return EDefaultIOProcessInputKind::Albedo;
		case EResourceName::Normal:	return EDefaultIOProcessInputKind::Normal;
		case EResourceName::Flow:	return EDefaultIOProcessInputKind::Flow;
		case EResourceName::Output:	return EDefaultIOProcessInputKind::Output;
	}

	// There should be a case for every resource name
	checkNoEntry();
	
	return EDefaultIOProcessInputKind::Color;
}

void AddPreOrPostProcess(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef InputTexture,
	EResourceName TensorName,
	int32 FrameIdx,
	FRDGTextureRef OutputTexture)
{
	using namespace UE::NNEDenoiserShaders::Internal;

	const FIntVector Size = InputTexture->Desc.GetSize();
	check(Size == OutputTexture->Desc.GetSize());

	FDefaultIOProcessCS::FParameters *ShaderParameters = GraphBuilder.AllocParameters<FDefaultIOProcessCS::FParameters>();
	ShaderParameters->Width = Size.X;
	ShaderParameters->Height = Size.Y;
	ShaderParameters->InputTexture = InputTexture;
	ShaderParameters->OutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutputTexture));

	FDefaultIOProcessCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FDefaultIOProcessCS::FDefaultIOProcessInputKind>(GetInputKind(TensorName));

	FIntVector ThreadGroupCount = FIntVector(
		FMath::DivideAndRoundUp(Size.X, FDefaultIOProcessConstants::THREAD_GROUP_SIZE),
		FMath::DivideAndRoundUp(Size.Y, FDefaultIOProcessConstants::THREAD_GROUP_SIZE),
		1);

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FDefaultIOProcessCS> Shader(GlobalShaderMap, PermutationVector);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEDenoiserDefaultIOProcess, "NNEDenoiser.DefaultIOProcess");

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NNEDenoiser.DefaultIOProcess"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		Shader,
		ShaderParameters,
		ThreadGroupCount);
}

void AddReadInputPass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef InputTexture,
	FRDGBufferUAVRef OutputBufferUAV,
	const TArray<FChannelMapping>& ChannelMapping)
{
	using namespace UE::NNEDenoiserShaders::Internal;

	if (ChannelMapping.IsEmpty())
	{
		UE_LOGF(LogNNEDenoiser, Warning, "AddReadInputPass: ChannelMapping is empty. Nothing to do!");
		return;
	}
	
	if (ChannelMapping.Num() >= FMappedCopyConstants::MAX_NUM_MAPPED_CHANNELS)
	{
		UE_LOGF(LogNNEDenoiser, Warning, "AddReadInputPass: ChannelMapping has too many entries!");
		return;
	}

	const FIntVector Size = InputTexture->Desc.GetSize();

	FTextureBufferMappedCopyCS::FParameters *ReadInputParameters = GraphBuilder.AllocParameters<FTextureBufferMappedCopyCS::FParameters>();
	ReadInputParameters->Width = Size.X;
	ReadInputParameters->Height = Size.Y;
	ReadInputParameters->InputTexture = InputTexture;
	ReadInputParameters->OutputBuffer = OutputBufferUAV;
	for (int32 Idx = 0; Idx < ChannelMapping.Num(); Idx++)
	{
		ReadInputParameters->OutputChannel_InputChannel_Unused_Unused[Idx] = { ChannelMapping[Idx].TensorChannel, ChannelMapping[Idx].ResourceChannel, 0, 0 };
	}

	const EDataType InputDataType = GetDenoiserShaderDataType(InputTexture->Desc.Format);
	const EDataType OutputDataType = GetDenoiserShaderDataType(OutputBufferUAV->Desc.Format);

	FTextureBufferMappedCopyCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FTextureBufferMappedCopyCS::FInputDataType>(InputDataType);
	PermutationVector.Set<FTextureBufferMappedCopyCS::FOutputDataType>(OutputDataType);
	PermutationVector.Set<FTextureBufferMappedCopyCS::FNumMappedChannels>(ChannelMapping.Num());

	FIntVector ReadInputThreadGroupCount = FIntVector(
		FMath::DivideAndRoundUp(Size.X, FMappedCopyConstants::THREAD_GROUP_SIZE),
		FMath::DivideAndRoundUp(Size.Y, FMappedCopyConstants::THREAD_GROUP_SIZE),
		1);

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FTextureBufferMappedCopyCS> ReadInputShader(GlobalShaderMap, PermutationVector);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEDenoiserReadInput, "NNEDenoiser.ReadInput");

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NNEDenoiser.ReadInput"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ReadInputShader,
		ReadInputParameters,
		ReadInputThreadGroupCount);
}

void AddWriteOutputPass(
	FRDGBuilder& GraphBuilder,
	FRDGBufferRef InputBuffer,
	EPixelFormat BufferFormat,
	FRDGTextureRef OutputTexture,
	UE::NNEDenoiserShaders::Internal::EDataType DataType,
	const TArray<FChannelMapping>& ChannelMapping)
{
	using namespace UE::NNEDenoiserShaders::Internal;

	const FIntVector Size = OutputTexture->Desc.GetSize();

	FBufferTextureMappedCopyCS::FParameters *WriteOutputParameters = GraphBuilder.AllocParameters<FBufferTextureMappedCopyCS::FParameters>();
	WriteOutputParameters->Width = Size.X;
	WriteOutputParameters->Height = Size.Y;
	WriteOutputParameters->InputBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputBuffer, BufferFormat));
	WriteOutputParameters->OutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutputTexture));
	for (int32 Idx = 0; Idx < ChannelMapping.Num(); Idx++)
	{
		WriteOutputParameters->OutputChannel_InputChannel_Unused_Unused[Idx] = { ChannelMapping[Idx].ResourceChannel, ChannelMapping[Idx].TensorChannel, 0, 0 };
	}

	const EDataType InputDataType = GetDenoiserShaderDataType(BufferFormat);
	const EDataType OutputDataType = GetDenoiserShaderDataType(OutputTexture->Desc.Format);

	FBufferTextureMappedCopyCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FBufferTextureMappedCopyCS::FInputDataType>(InputDataType);
	PermutationVector.Set<FBufferTextureMappedCopyCS::FOutputDataType>(OutputDataType);
	PermutationVector.Set<FBufferTextureMappedCopyCS::FNumMappedChannels>(ChannelMapping.Num());

	FIntVector WriteOutputThreadGroupCount = FIntVector(
		FMath::DivideAndRoundUp(Size.X, FMappedCopyConstants::THREAD_GROUP_SIZE),
		FMath::DivideAndRoundUp(Size.Y, FMappedCopyConstants::THREAD_GROUP_SIZE),
		1);

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FBufferTextureMappedCopyCS> WriteOutputShader(GlobalShaderMap, PermutationVector);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEDenoiserWriteOutput, "NNEDenoiser.WriteOutput");

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NNEDenoiser.WriteOutput"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		WriteOutputShader,
		WriteOutputParameters,
		WriteOutputThreadGroupCount);
}

void AddReadInputPassForKind(
	FRDGBuilder& GraphBuilder,
	const IResourceAccess& ResourceAccess,
	EResourceName TensorName,
	const FResourceMapping& ResourceMapping,
	FRDGBufferUAVRef BufferUAV)
{
	for (const TPair<int32, TArray<FChannelMapping>>& ChannelMappingKeyValue : ResourceMapping.GetChannelMappingPerFrame(TensorName))
	{
		const int32 FrameIdx = -ChannelMappingKeyValue.Key;
		FRDGTextureRef InputTexture = ResourceAccess.GetTexture(TensorName, FrameIdx);

		AddReadInputPass(GraphBuilder, InputTexture, BufferUAV, ChannelMappingKeyValue.Value);
	}
};

} // namespace IOProcessBaseHelper

bool FInputProcessBase::Validate(const IModelInstance& ModelInstance, FIntPoint Extent) const
{
	SCOPED_NAMED_EVENT_TEXT("NNEDenoiser.Validate", FColor::Magenta);

	checkf(Extent == FIntPoint(-1, -1) || (Extent.X >= 0 && Extent.Y >= 0), TEXT("Extent should be either fully symbolic or set!"));

	const int32 NumBatches = 1;

	UE_LOGF(LogNNEDenoiser, Log, "Validate model for extent %dx%d...", Extent.X, Extent.Y);

	if (Extent == FIntPoint(-1, -1))
	{
		TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = ModelInstance.GetInputTensorDescs();
		if (InputTensorDescs.Num() != InputLayout.Num())
		{
			UE_LOGF(LogNNEDenoiser, Error, "Wrong number of inputs (expected %d, got %d)!", InputLayout.Num(), InputTensorDescs.Num())
			return false;
		}

		TArray<UE::NNE::FTensorShape> InputShapes;
		for (int32 Idx = 0; Idx < InputTensorDescs.Num(); Idx++)
		{
			TConstArrayView<int32> InputSymbolicTensorShapeData = InputTensorDescs[Idx].GetShape().GetData();
			const TArray<int32, TInlineAllocator<4>> RequiredInputShapeData = { NumBatches, InputLayout.NumChannels(Idx), Extent.Y, Extent.X };

			if (!IsTensorShapeValid(InputSymbolicTensorShapeData, RequiredInputShapeData, TEXT("Input")))
			{
				return false;
			}
		}
	}
	else
	{
		TConstArrayView<UE::NNE::FTensorShape> InputTensorShapes = ModelInstance.GetInputTensorShapes();
		if (InputTensorShapes.Num() != InputLayout.Num())
		{
			UE_LOGF(LogNNEDenoiser, Error, "Wrong number of inputs (expected %d, got %d)!", InputLayout.Num(), InputTensorShapes.Num())
			return false;
		}

		for (int32 Idx = 0; Idx < InputTensorShapes.Num(); Idx++)
		{
			TConstArrayView<uint32> InputTensorShapeData = InputTensorShapes[Idx].GetData();
			const TArray<int32, TInlineAllocator<4>> RequiredInputShapeData = { NumBatches, InputLayout.NumChannels(Idx), Extent.Y, Extent.X };

			if (!IsTensorShapeValid(InputTensorShapeData, RequiredInputShapeData, TEXT("Input")))
			{
				return false;
			}
		}
	}

	return true;
}

bool FInputProcessBase::Prepare(IModelInstance& ModelInstance, FIntPoint Extent) const
{
	SCOPED_NAMED_EVENT_TEXT("NNEDenoiser.Prepare", FColor::Magenta);

	if (!Validate(ModelInstance, {-1, -1}))
	{
		return false;
	}

	const int32 NumBatches = 1;

	UE_LOGF(LogNNEDenoiser, Log, "Configure model for extent %dx%d...", Extent.X, Extent.Y);

	TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = ModelInstance.GetInputTensorDescs();
	
	UE_LOGF(LogNNEDenoiser, Log, "Input shapes (set):");

	TArray<UE::NNE::FTensorShape> InputShapes;
	for (int32 Idx = 0; Idx < InputTensorDescs.Num(); Idx++)
	{
		TConstArrayView<int32> InputSymbolicTensorShapeData = InputTensorDescs[Idx].GetShape().GetData();

		const FIntVector4 ModelInputShape = { 1, InputSymbolicTensorShapeData[1], Extent.Y, Extent.X };

		InputShapes.Add(UE::NNE::FTensorShape::Make({ (uint32)ModelInputShape.X, (uint32)ModelInputShape.Y, (uint32)ModelInputShape.Z, (uint32)ModelInputShape.W }));

		UE_LOGF(LogNNEDenoiser, Log, "%d: (%d, %d, %d, %d)", Idx, ModelInputShape.X, ModelInputShape.Y, ModelInputShape.Z, ModelInputShape.W)
	}

	const IModelInstance::ESetInputTensorShapesStatus Status = ModelInstance.SetInputTensorShapes(InputShapes);
	if (Status != IModelInstance::ESetInputTensorShapesStatus::Ok)
	{
		UE_LOGF(LogNNEDenoiser, Error, "Could not configure model instance (ModelInstance.SetInputTensorShapes() failed)!")
		return false;
	}

	if (!Validate(ModelInstance, Extent))
	{
		return false;
	}

	return true;
}

int32 FInputProcessBase::NumFrames(EResourceName Name) const
{
	return InputLayout.NumFrames(Name);
}

void FInputProcessBase::AddPasses(
		FRDGBuilder& GraphBuilder,
		TConstArrayView<UE::NNE::FTensorDesc> TensorDescs,
		TConstArrayView<UE::NNE::FTensorShape> TensorShapes,
		const IResourceAccess& ResourceAccess,
		TConstArrayView<FRDGBufferRef> OutputBuffers) const
{
	const UEnum* ResourceNameEnum = StaticEnum<EResourceName>();
	for (int32 I = 0; I < ResourceNameEnum->NumEnums(); I++)
	{
		const EResourceName TensorName = (EResourceName)ResourceNameEnum->GetValueByIndex(I);
		const int32 NumFrames = InputLayout.NumFrames(TensorName);

		for (int32 J = 0; J < NumFrames; J++)
		{
			if (!HasPreprocessInput(TensorName, J))
			{
				continue;
			}

			FRDGTextureRef InputTexture = ResourceAccess.GetTexture(TensorName, J);
			FRDGTextureRef PreprocessedInputTexture = ResourceAccess.GetIntermediateTexture(TensorName, J);

			PreprocessInput(GraphBuilder, InputTexture, TensorName, J, PreprocessedInputTexture);
		}
	}

	for (int32 I = 0; I < TensorDescs.Num(); I++)
	{
		WriteInputBuffer(GraphBuilder, TensorDescs[I], TensorShapes[I], ResourceAccess, InputLayout.GetChecked(I), OutputBuffers[I]);
	}
}

bool FInputProcessBase::HasPreprocessInput(EResourceName TensorName, int32 FrameIdx) const
{
	check(FrameIdx == 0);

	switch(TensorName)
	{
		case EResourceName::Color:
		case EResourceName::Albedo:
		case EResourceName::Normal:
			return true;
	}
	return false;
}

void FInputProcessBase::PreprocessInput(
		FRDGBuilder& GraphBuilder,
		FRDGTextureRef Texture,
		EResourceName TensorName,
		int32 FrameIdx,
		FRDGTextureRef PreprocessedTexture) const
{
	IOProcessBaseHelper::AddPreOrPostProcess(GraphBuilder, Texture, TensorName, FrameIdx, PreprocessedTexture);

	if (TransferFunction.IsValid() && TensorName == EResourceName::Color)
	{
		TransferFunction->RDGForward(GraphBuilder, PreprocessedTexture, PreprocessedTexture);
	}
}

void FInputProcessBase::WriteInputBuffer(
	FRDGBuilder& GraphBuilder,
	const UE::NNE::FTensorDesc& TensorDesc,
	const UE::NNE::FTensorShape& TensorShape,
	const IResourceAccess& ResourceAccess,
	const FResourceMapping& ResourceMapping,
	FRDGBufferRef Buffer) const
{
	const ENNETensorDataType TensorDataType = TensorDesc.GetDataType();

	FRDGBufferUAVRef BufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Buffer, GetBufferFormat(TensorDataType)));

	IOProcessBaseHelper::AddReadInputPassForKind(GraphBuilder, ResourceAccess, EResourceName::Color, ResourceMapping, BufferUAV);
	IOProcessBaseHelper::AddReadInputPassForKind(GraphBuilder, ResourceAccess, EResourceName::Albedo, ResourceMapping, BufferUAV);
	IOProcessBaseHelper::AddReadInputPassForKind(GraphBuilder, ResourceAccess, EResourceName::Normal, ResourceMapping, BufferUAV);
	IOProcessBaseHelper::AddReadInputPassForKind(GraphBuilder, ResourceAccess, EResourceName::Flow, ResourceMapping, BufferUAV);
	IOProcessBaseHelper::AddReadInputPassForKind(GraphBuilder, ResourceAccess, EResourceName::Output, ResourceMapping, BufferUAV);
}

bool FOutputProcessBase::Validate(const IModelInstance& ModelInstance, FIntPoint Extent) const
{
	SCOPED_NAMED_EVENT_TEXT("NNEDenoiser.Prepare", FColor::Magenta);

	const int32 NumBatches = 1;

	TConstArrayView<UE::NNE::FTensorDesc> OutputTensorDescs = ModelInstance.GetOutputTensorDescs();
	if (OutputTensorDescs.Num() != OutputLayout.Num())
	{
		UE_LOGF(LogNNEDenoiser, Error, "Wrong number of outputs (expected %d, got %d)!", OutputLayout.Num(), OutputTensorDescs.Num())
		return false;
	}

	for (int32 Idx = 0; Idx < OutputTensorDescs.Num(); Idx++)
	{
		TConstArrayView<int32> OutputSymbolicShapeData = OutputTensorDescs[Idx].GetShape().GetData();
		const TArray<int32, TInlineAllocator<4>> RequiredOutputShapeData = { NumBatches, OutputLayout.NumChannels(Idx), -1, -1 };

		if (!IsTensorShapeValid(OutputSymbolicShapeData, RequiredOutputShapeData, TEXT("Output")))
		{
			return false;
		}
	}

	TConstArrayView<UE::NNE::FTensorShape> OutputShapes = ModelInstance.GetOutputTensorShapes();
	if (!OutputShapes.IsEmpty() && OutputShapes.Num() != OutputLayout.Num())
	{
		UE_LOGF(LogNNEDenoiser, Error, "Wrong number of output shapes!");
		return false;
	}

	if (OutputShapes.IsEmpty() && OutputLayout.Num() > 0)
	{
		UE_LOGF(LogNNEDenoiser, Log, "Output shapes not resolved yet");
		return true;
	}
	
	UE_LOGF(LogNNEDenoiser, Log, "Output shapes (resolved):");

	for (int32 Idx = 0; Idx < OutputTensorDescs.Num(); Idx++)
	{
		TConstArrayView<uint32> OutputShapeData = OutputShapes[Idx].GetData();
		const TArray<int32, TInlineAllocator<4>> RequiredOutputShapeData = { NumBatches, OutputLayout.NumChannels(Idx), -1, -1 };

		if (!IsTensorShapeValid(OutputShapeData, RequiredOutputShapeData, TEXT("Output")))
		{
			return false;
		}
	
		UE_LOGF(LogNNEDenoiser, Log, "%d: (%d, %d, %d, %d)", Idx, OutputShapeData[0], OutputShapeData[1], OutputShapeData[2], OutputShapeData[3]);
	}

	return true;
}

void FOutputProcessBase::AddPasses(
	FRDGBuilder& GraphBuilder,
	TConstArrayView<UE::NNE::FTensorDesc> TensorDescs,
	TConstArrayView<UE::NNE::FTensorShape> TensorShapes,
	const IResourceAccess& ResourceAccess,
	TConstArrayView<FRDGBufferRef> Buffers,
	FRDGTextureRef OutputTexture) const
{
	for (int32 I = 0; I < TensorDescs.Num(); I++)
	{
		// outputs might be discarded
		const FResourceMapping* ResourceMapping = OutputLayout.Get(I);
		if (ResourceMapping && ResourceMapping->GetChannelMappingPerFrame(EResourceName::Output).Num() == 1)
		{
			ReadOutputBuffer(GraphBuilder, TensorDescs[I], TensorShapes[I], ResourceAccess, Buffers[I], *ResourceMapping, OutputTexture);
		}
	}

	const EResourceName TensorName = EResourceName::Output;
	const int32 FrameIdx = 0;

	if (HasPostprocessOutput(TensorName, FrameIdx))
	{
		FRDGTextureRef PostprocessInputTexture = ResourceAccess.GetTexture(EResourceName::Output, FrameIdx);
		FRDGTextureRef PostprocessOutputTexture = ResourceAccess.GetIntermediateTexture(EResourceName::Output, FrameIdx);
		
		PostprocessOutput(GraphBuilder, PostprocessInputTexture, PostprocessOutputTexture);
	}
}

bool FOutputProcessBase::HasPostprocessOutput(EResourceName TensorName, int32 FrameIdx) const
{
	check(FrameIdx == 0);

	return TensorName == EResourceName::Output;
}

void FOutputProcessBase::ReadOutputBuffer(
	FRDGBuilder& GraphBuilder,
	const UE::NNE::FTensorDesc& TensorDesc,
	const UE::NNE::FTensorShape& TensorShape,
	const IResourceAccess& ResourceAccess,
	FRDGBufferRef Buffer,
	const FResourceMapping& ResourceMapping,
	FRDGTextureRef Texture) const
{
	const ENNETensorDataType TensorDataType = TensorDesc.GetDataType();
	const NNEDenoiserShaders::Internal::EDataType DataType = GetDenoiserShaderDataType(TensorDataType);
	const FIntPoint BufferSize(TensorShape.GetData()[3], TensorShape.GetData()[2]);
		
	TMap<int32, TArray<FChannelMapping>> ChannelMappingPerFrame = ResourceMapping.GetChannelMappingPerFrame(EResourceName::Output);
	checkf(ChannelMappingPerFrame.Num() == 1, TEXT("Invalid output mapping"));
	const TArray<FChannelMapping>& ChannelMapping = ChannelMappingPerFrame.CreateConstIterator().Value();

	const EPixelFormat BufferFormat = GetBufferFormat(TensorDataType);

	IOProcessBaseHelper::AddWriteOutputPass(GraphBuilder, Buffer, BufferFormat, Texture, DataType, ChannelMapping);
}

void FOutputProcessBase::PostprocessOutput(
		FRDGBuilder& GraphBuilder,
		FRDGTextureRef Texture,
		FRDGTextureRef PostprocessedTexture) const
{
	if (TransferFunction.IsValid())
	{
		TransferFunction->RDGInverse(GraphBuilder, Texture, PostprocessedTexture);
	}

	IOProcessBaseHelper::AddPreOrPostProcess(GraphBuilder, PostprocessedTexture, EResourceName::Output, 0, PostprocessedTexture);
}

} // namespace UE::NNEDenoiser::Private