// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEDenoiserIOProcess.h"
#include "NNEDenoiserResourceMapping.h"

namespace UE::NNEDenoiser::Private
{

class ITransferFunction;

class FInputProcessBase : public IInputProcess
{
public:
	FInputProcessBase(FResourceMappingList InputLayout, TSharedPtr<ITransferFunction> TransferFunction) : InputLayout(InputLayout), TransferFunction(TransferFunction)
	{

	}

	virtual ~FInputProcessBase() = default;

	virtual bool Validate(const IModelInstance& ModelInstance, FIntPoint Extent) const override;

	virtual bool Prepare(IModelInstance& ModelInstance, FIntPoint Extent) const override;

	virtual int32 NumFrames(EResourceName Name) const override;

	void AddPasses(
		FRDGBuilder& GraphBuilder,
		TConstArrayView<UE::NNE::FTensorDesc> TensorDescs,
		TConstArrayView<UE::NNE::FTensorShape> TensorShapes,
		const IResourceAccess& ResourceAccess,
		TConstArrayView<FRDGBufferRef> OutputBuffers) const override;

protected:
	virtual bool HasPreprocessInput(EResourceName TensorName, int32 FrameIdx) const;

	virtual void PreprocessInput(
		FRDGBuilder& GraphBuilder,
		FRDGTextureRef Texture,
		EResourceName TensorName,
		int32 FrameIdx,
		FRDGTextureRef PreprocessedTexture) const;

	virtual void WriteInputBuffer(
		FRDGBuilder& GraphBuilder,
		const UE::NNE::FTensorDesc& TensorDesc,
		const UE::NNE::FTensorShape& TensorShape,
		const IResourceAccess& ResourceAccess,
		const FResourceMapping& ResourceMapping,
		FRDGBufferRef Buffer) const;

private:
	FResourceMappingList InputLayout;
	TSharedPtr<ITransferFunction> TransferFunction;
};

class FOutputProcessBase : public IOutputProcess
{
public:
	FOutputProcessBase(FResourceMappingList OutputLayout, TSharedPtr<ITransferFunction> TransferFunction) : OutputLayout(OutputLayout), TransferFunction(TransferFunction)
	{

	}

	virtual ~FOutputProcessBase() = default;

	virtual bool Validate(const IModelInstance& ModelInstance, FIntPoint Extent) const override;

	void AddPasses(
		FRDGBuilder& GraphBuilder,
		TConstArrayView<UE::NNE::FTensorDesc> TensorDescs,
		TConstArrayView<UE::NNE::FTensorShape> TensorShapes,
		const IResourceAccess& ResourceAccess,
		TConstArrayView<FRDGBufferRef> Buffers,
		FRDGTextureRef OutputTexture) const override;

protected:
	virtual bool HasPostprocessOutput(EResourceName TensorName, int32 FrameIdx) const;

	virtual void ReadOutputBuffer(
		FRDGBuilder& GraphBuilder,
		const UE::NNE::FTensorDesc& TensorDesc,
		const UE::NNE::FTensorShape& TensorShape,
		const IResourceAccess& ResourceAccess,
		FRDGBufferRef Buffer,
		const FResourceMapping& ResourceMapping,
		FRDGTextureRef Texture) const;

	virtual void PostprocessOutput(
		FRDGBuilder& GraphBuilder,
		FRDGTextureRef Texture,
		FRDGTextureRef PostprocessedTexture) const;

private:
	FResourceMappingList OutputLayout;
	TSharedPtr<ITransferFunction> TransferFunction;
};

} // namespace UE::NNEDenoiser::Private