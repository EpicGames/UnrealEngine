// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDG.h"

namespace UE::NNEDenoiser::Private
{

class IModelInstance : public NNE::IModelInstanceRDG
{
public:
	using ESetInputTensorShapesStatus = NNE::IModelInstanceRDG::ESetInputTensorShapesStatus;
	using EEnqueueRDGStatus = NNE::IModelInstanceRDG::EEnqueueRDGStatus;

	virtual ~IModelInstance() = default;
};

}