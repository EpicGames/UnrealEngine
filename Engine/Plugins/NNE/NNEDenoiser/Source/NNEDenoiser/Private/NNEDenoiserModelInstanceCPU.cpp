// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserModelInstanceCPU.h"
#include "NNE.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserUtils.h"
#include "NNEModelData.h"
#include "NNERuntimeCPU.h"
#include "RenderGraphBuilder.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHITypes.h"
#include "UObject/UObjectGlobals.h"

BEGIN_SHADER_PARAMETER_STRUCT(FNNEDenoiserModelInstanceCPUTextureParameters, )
	RDG_BUFFER_ACCESS_ARRAY(InputBuffers)
	RDG_BUFFER_ACCESS_ARRAY(OutputBuffers)
END_SHADER_PARAMETER_STRUCT()

namespace UE::NNEDenoiser::Private
{

	TUniquePtr<FModelInstanceCPU> FModelInstanceCPU::Make(UNNEModelData& ModelData, const FString& RuntimeName)
	{
		check(!RuntimeName.IsEmpty());

		TWeakInterfacePtr<INNERuntimeCPU> Runtime = UE::NNE::GetRuntime<INNERuntimeCPU>(RuntimeName);
		if (!Runtime.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model instance. No CPU runtime '%ls' found. Valid CPU runtimes are: ", *RuntimeName);
			for (const FString& Name : UE::NNE::GetAllRuntimeNames<INNERuntimeCPU>())
			{
				UE_LOGF(LogNNEDenoiser, Log, "- %ls", *Name);
			}
			return {};
		}

		if (Runtime->CanCreateModelCPU(&ModelData) != INNERuntimeCPU::ECanCreateModelCPUStatus::Ok)
		{
			UE_LOGF(LogNNEDenoiser, Log, "%ls on CPU can not create model", *RuntimeName);
			return {};
		}

		TSharedPtr<UE::NNE::IModelCPU> Model = Runtime->CreateModelCPU(&ModelData);
		if (!Model.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model using %ls on CPU", *RuntimeName);
			return {};
		}

		TSharedPtr<UE::NNE::IModelInstanceCPU> ModelInstance = Model->CreateModelInstanceCPU();
		if (!ModelInstance.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model instance using %ls on CPU", *RuntimeName);
			return {};
		}

		return MakeUnique<FModelInstanceCPU>(ModelInstance.ToSharedRef());
	}

	FModelInstanceCPU::FModelInstanceCPU(TSharedRef<UE::NNE::IModelInstanceCPU> ModelInstance) :
			ModelInstance(ModelInstance)
	{

	}

	FModelInstanceCPU::~FModelInstanceCPU()
	{
		
	}

	TConstArrayView<NNE::FTensorDesc> FModelInstanceCPU::GetInputTensorDescs() const
	{
		return ModelInstance->GetInputTensorDescs();
	}

	TConstArrayView<NNE::FTensorDesc> FModelInstanceCPU::GetOutputTensorDescs() const
	{
		return ModelInstance->GetOutputTensorDescs();
	}

	TConstArrayView<NNE::FTensorShape> FModelInstanceCPU::GetInputTensorShapes() const
	{
		return ModelInstance->GetInputTensorShapes();
	}

	TConstArrayView<NNE::FTensorShape> FModelInstanceCPU::GetOutputTensorShapes() const
	{
		return ModelInstance->GetOutputTensorShapes();
	}

	FModelInstanceCPU::ESetInputTensorShapesStatus FModelInstanceCPU::SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes)
	{
		NNE::IModelInstanceCPU::ESetInputTensorShapesStatus Status = ModelInstance->SetInputTensorShapes(InInputShapes);

		return Status == NNE::IModelInstanceCPU::ESetInputTensorShapesStatus::Ok ? ESetInputTensorShapesStatus::Ok : ESetInputTensorShapesStatus::Fail;
	}

	FModelInstanceCPU::EEnqueueRDGStatus FModelInstanceCPU::EnqueueRDG(FRDGBuilder &GraphBuilder, TConstArrayView<NNE::FTensorBindingRDG> Inputs, TConstArrayView<NNE::FTensorBindingRDG> Outputs)
	{
		FNNEDenoiserModelInstanceCPUTextureParameters* DenoiserParameters = GraphBuilder.AllocParameters<FNNEDenoiserModelInstanceCPUTextureParameters>();
		for (const NNE::FTensorBindingRDG& Binding : Inputs)
		{
			DenoiserParameters->InputBuffers.Emplace(Binding.Buffer, ERHIAccess::CopySrc);
		}
		for (const NNE::FTensorBindingRDG& Binding : Outputs)
		{
			DenoiserParameters->OutputBuffers.Emplace(Binding.Buffer, ERHIAccess::CopyDest);
		}
		
		GraphBuilder.AddPass(RDG_EVENT_NAME("NNEDenoiser.DenoiseCPU"), DenoiserParameters, ERDGPassFlags::Readback,
		[this, DenoiserParameters](FRHICommandListImmediate& RHICmdList)
		{
			SCOPED_NAMED_EVENT_TEXT("FModelInstanceCPU::DenoisePass", FColor::Magenta);
			
#if WITH_EDITOR
			// NOTE: the time will include the transfer from GPU to CPU which will include waiting for the GPU pipeline to complete
			uint64 FilterExecuteTime = 0;
			FilterExecuteTime -= FPlatformTime::Cycles64();
#endif

			ScratchInputBuffers.SetNum(DenoiserParameters->InputBuffers.Num());
			TArray<NNE::FTensorBindingCPU> InputBindings;
			for (int32 Idx = 0; Idx < DenoiserParameters->InputBuffers.Num(); Idx++)
			{
				ScratchInputBuffers[Idx].SetNumUninitialized(DenoiserParameters->InputBuffers[Idx].GetBuffer()->GetSize());

				InputBindings.Emplace(NNE::FTensorBindingCPU{(void *)ScratchInputBuffers[Idx].GetData(), (uint64)ScratchInputBuffers[Idx].Num()});
			}

			ScratchOutputBuffers.SetNum(DenoiserParameters->OutputBuffers.Num());
			TArray<NNE::FTensorBindingCPU> OutputBindings;
			for (int32 Idx = 0; Idx < DenoiserParameters->OutputBuffers.Num(); Idx++)
			{
				ScratchOutputBuffers[Idx].SetNumUninitialized(DenoiserParameters->OutputBuffers[Idx].GetBuffer()->GetSize());

				OutputBindings.Emplace(NNE::FTensorBindingCPU{(void *)ScratchOutputBuffers[Idx].GetData(), (uint64)ScratchOutputBuffers[Idx].Num()});
			}

			for (int32 Idx = 0; Idx < DenoiserParameters->InputBuffers.Num(); Idx++)
			{
				FRHIBuffer* Buffer = DenoiserParameters->InputBuffers[Idx].GetBuffer()->GetRHI();
				const int32 Count = static_cast<int32>(DenoiserParameters->InputBuffers[Idx]->GetSize());
				CopyBufferFromGPUToCPU(RHICmdList, Buffer, Count, ScratchInputBuffers[Idx]);
			}

			NNE::IModelInstanceCPU::ERunSyncStatus Status = ModelInstance->RunSync(InputBindings, OutputBindings);
			checkf(Status == NNE::IModelInstanceCPU::ERunSyncStatus::Ok, TEXT("RunSync failed with status %d"), static_cast<int>(Status))

			for (int32 Idx = 0; Idx < DenoiserParameters->OutputBuffers.Num(); Idx++)
			{
				FRHIBuffer* Buffer = DenoiserParameters->OutputBuffers[Idx].GetBuffer()->GetRHI();
				const int32 Count = static_cast<int32>(DenoiserParameters->OutputBuffers[Idx]->GetSize());
				CopyBufferFromCPUToGPU(RHICmdList, ScratchOutputBuffers[Idx], Count, Buffer);
			}

#if WITH_EDITOR
		FilterExecuteTime += FPlatformTime::Cycles64();
		const double FilterExecuteTimeMS = 1000.0 * FPlatformTime::ToSeconds64(FilterExecuteTime);
		UE_LOGF(LogNNEDenoiser, Log, "Denoised on CPU in %.2f ms", FilterExecuteTimeMS);
#endif
		});

		return EEnqueueRDGStatus::Ok;
	}

}