// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEDenoiserModelInstance.h"
#include "NNEModelData.h"

class FRDGBuilder;

namespace UE::NNE
{
	class IModelInstanceCPU;
}

namespace UE::NNEDenoiser::Private
{

class FModelInstanceCPU : public IModelInstance
{
public:
	explicit FModelInstanceCPU(TSharedRef<UE::NNE::IModelInstanceCPU> ModelInstance);

	static TUniquePtr<FModelInstanceCPU> Make(UNNEModelData& ModelData, const FString& RuntimeName);

	virtual ~FModelInstanceCPU();

	virtual TConstArrayView<NNE::FTensorDesc> GetInputTensorDescs() const override;

	virtual TConstArrayView<NNE::FTensorDesc> GetOutputTensorDescs() const override;

	virtual TConstArrayView<NNE::FTensorShape> GetInputTensorShapes() const override;

	virtual TConstArrayView<NNE::FTensorShape> GetOutputTensorShapes() const override;

	virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes) override;

	virtual EEnqueueRDGStatus EnqueueRDG(FRDGBuilder &GraphBuilder, TConstArrayView<NNE::FTensorBindingRDG> Inputs, TConstArrayView<NNE::FTensorBindingRDG> Outputs) override;

private:
	TSharedRef<UE::NNE::IModelInstanceCPU> ModelInstance;

	TArray<TArray<uint8>> ScratchInputBuffers;
	TArray<TArray<uint8>> ScratchOutputBuffers;
};
	
}