// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserModelInstanceGPU.h"
#include "NNE.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserUtils.h"
#include "NNEModelData.h"
#include "NNERuntimeGPU.h"
#include "PathTracingDenoiser.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderGraphBuilder.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHITypes.h"
#include "UObject/UObjectGlobals.h"

DECLARE_GPU_STAT_NAMED(FModelInstanceGPU, TEXT("NNEDenoiser.ModelInstanceGPU"));

BEGIN_SHADER_PARAMETER_STRUCT(FNNEDenoiserModelInstanceGPUTextureParameters, )
	RDG_BUFFER_ACCESS_ARRAY(InputBuffers)
	RDG_BUFFER_ACCESS_ARRAY(OutputBuffers)
END_SHADER_PARAMETER_STRUCT()

namespace UE::NNEDenoiser::Private
{

	TUniquePtr<FModelInstanceGPU> FModelInstanceGPU::Make(UNNEModelData& ModelData, const FString& RuntimeName)
	{
		check(!RuntimeName.IsEmpty());

		TWeakInterfacePtr<INNERuntimeGPU> Runtime = UE::NNE::GetRuntime<INNERuntimeGPU>(RuntimeName);
		if (!Runtime.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model instance. No GPU runtime '%ls' found. Valid GPU runtimes are: ", *RuntimeName);
			for (const FString& Name : UE::NNE::GetAllRuntimeNames<INNERuntimeGPU>())
			{
				UE_LOGF(LogNNEDenoiser, Log, "- %ls", *Name);
			}
			return {};
		}

		if (Runtime->CanCreateModelGPU(&ModelData) != INNERuntimeGPU::ECanCreateModelGPUStatus::Ok)
		{
			UE_LOGF(LogNNEDenoiser, Log, "%ls on GPU can not create model", *RuntimeName);
			return {};
		}

		TSharedPtr<UE::NNE::IModelGPU> Model = Runtime->CreateModelGPU(&ModelData);
		if (!Model.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model using %ls on GPU", *RuntimeName);
			return {};
		}

		TSharedPtr<UE::NNE::IModelInstanceGPU> ModelInstance = Model->CreateModelInstanceGPU();
		if (!ModelInstance.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model instance using %ls on GPU", *RuntimeName);
			return {};
		}

		return MakeUnique<FModelInstanceGPU>(ModelInstance.ToSharedRef());
	}

	FModelInstanceGPU::FModelInstanceGPU(TSharedRef<UE::NNE::IModelInstanceGPU> ModelInstance) :
		ModelInstance(ModelInstance)
	{

	}

	FModelInstanceGPU::~FModelInstanceGPU()
	{
		
	}

	TConstArrayView<NNE::FTensorDesc> FModelInstanceGPU::GetInputTensorDescs() const
	{
		return ModelInstance->GetInputTensorDescs();
	}

	TConstArrayView<NNE::FTensorDesc> FModelInstanceGPU::GetOutputTensorDescs() const
	{
		return ModelInstance->GetOutputTensorDescs();
	}

	TConstArrayView<NNE::FTensorShape> FModelInstanceGPU::GetInputTensorShapes() const
	{
		return ModelInstance->GetInputTensorShapes();
	}

	TConstArrayView<NNE::FTensorShape> FModelInstanceGPU::GetOutputTensorShapes() const
	{
		return ModelInstance->GetOutputTensorShapes();
	}

	FModelInstanceGPU::ESetInputTensorShapesStatus FModelInstanceGPU::SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes)
	{
		NNE::IModelInstanceGPU::ESetInputTensorShapesStatus Status = ModelInstance->SetInputTensorShapes(InInputShapes);

		return Status == NNE::IModelInstanceGPU::ESetInputTensorShapesStatus::Ok ? ESetInputTensorShapesStatus::Ok : ESetInputTensorShapesStatus::Fail;
	}

	FModelInstanceGPU::EEnqueueRDGStatus FModelInstanceGPU::EnqueueRDG(FRDGBuilder &GraphBuilder, TConstArrayView<NNE::FTensorBindingRDG> Inputs, TConstArrayView<NNE::FTensorBindingRDG> Outputs)
	{
		FNNEDenoiserModelInstanceGPUTextureParameters* DenoiserParameters = GraphBuilder.AllocParameters<FNNEDenoiserModelInstanceGPUTextureParameters>();
		for (const NNE::FTensorBindingRDG& Binding : Inputs)
		{
			DenoiserParameters->InputBuffers.Emplace(Binding.Buffer, ERHIAccess::CopySrc);
		}
		for (const NNE::FTensorBindingRDG& Binding : Outputs)
		{
			DenoiserParameters->OutputBuffers.Emplace(Binding.Buffer, ERHIAccess::CopyDest);
		}

		RDG_EVENT_SCOPE_STAT(GraphBuilder, FModelInstanceGPU, "NNEDenoiser.ModelInstanceGPU");
		
		GraphBuilder.AddPass(RDG_EVENT_NAME("NNEDenoiser.DenoiseGPU"), DenoiserParameters, ERDGPassFlags::Readback,
		[this, DenoiserParameters](FRHICommandListImmediate& RHICmdList)
		{
			SCOPED_NAMED_EVENT_TEXT("FModelInstanceGPU::DenoisePass", FColor::Magenta);
			
#if WITH_EDITOR
			// NOTE: the time will include the transfer from GPU to CPU which will include waiting for the GPU pipeline to complete
			uint64 FilterExecuteTime = 0;
			FilterExecuteTime -= FPlatformTime::Cycles64();
#endif

			ScratchInputBuffers.SetNum(DenoiserParameters->InputBuffers.Num());
			TArray<NNE::FTensorBindingCPU> InputBindings;
			for (int32 Idx = 0; Idx < DenoiserParameters->InputBuffers.Num(); Idx++)
			{
				ScratchInputBuffers[Idx].SetNumUninitialized(DenoiserParameters->InputBuffers[Idx].GetBuffer()->GetSize());

				InputBindings.Emplace(NNE::FTensorBindingCPU{(void *)ScratchInputBuffers[Idx].GetData(), (uint64)ScratchInputBuffers[Idx].Num()});
			}

			ScratchOutputBuffers.SetNum(DenoiserParameters->OutputBuffers.Num());
			TArray<NNE::FTensorBindingCPU> OutputBindings;
			for (int32 Idx = 0; Idx < DenoiserParameters->OutputBuffers.Num(); Idx++)
			{
				ScratchOutputBuffers[Idx].SetNumUninitialized(DenoiserParameters->OutputBuffers[Idx].GetBuffer()->GetSize());

				OutputBindings.Emplace(NNE::FTensorBindingCPU{(void *)ScratchOutputBuffers[Idx].GetData(), (uint64)ScratchOutputBuffers[Idx].Num()});
			}

			for (int32 Idx = 0; Idx < DenoiserParameters->InputBuffers.Num(); Idx++)
			{
				FRHIBuffer* Buffer = DenoiserParameters->InputBuffers[Idx].GetBuffer()->GetRHI();
				const int32 Count = static_cast<int32>(DenoiserParameters->InputBuffers[Idx]->GetSize());
				CopyBufferFromGPUToCPU(RHICmdList, Buffer, Count, ScratchInputBuffers[Idx]);
			}

			NNE::IModelInstanceGPU::ERunSyncStatus Status = ModelInstance->RunSync(InputBindings, OutputBindings);
			checkf(Status == NNE::IModelInstanceGPU::ERunSyncStatus::Ok, TEXT("RunSync failed with status %d"), static_cast<int32>(Status))

			for (int32 Idx = 0; Idx < DenoiserParameters->OutputBuffers.Num(); Idx++)
			{
				FRHIBuffer* Buffer = DenoiserParameters->OutputBuffers[Idx].GetBuffer()->GetRHI();
				const int32 Count = static_cast<int32>(DenoiserParameters->OutputBuffers[Idx]->GetSize());
				CopyBufferFromCPUToGPU(RHICmdList, ScratchOutputBuffers[Idx], Count, Buffer);
			}

#if WITH_EDITOR
		FilterExecuteTime += FPlatformTime::Cycles64();
		const double FilterExecuteTimeMS = 1000.0 * FPlatformTime::ToSeconds64(FilterExecuteTime);
		UE_LOGF(LogNNEDenoiser, Log, "Denoised on GPU in %.2f ms", FilterExecuteTimeMS);
#endif
		});

		return EEnqueueRDGStatus::Ok;
	}

}