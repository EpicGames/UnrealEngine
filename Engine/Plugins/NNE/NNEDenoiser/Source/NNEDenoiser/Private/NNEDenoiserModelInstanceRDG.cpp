// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserModelInstanceRDG.h"
#include "NNE.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserUtils.h"
#include "NNEModelData.h"
#include "NNERuntimeRDG.h"
#include "PathTracingDenoiser.h"
#include "RenderGraphBuilder.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHITypes.h"
#include "UObject/UObjectGlobals.h"

namespace UE::NNEDenoiser::Private
{

	TUniquePtr<FModelInstanceRDG> FModelInstanceRDG::Make(UNNEModelData& ModelData, const FString& RuntimeName)
	{
		check(!RuntimeName.IsEmpty());
		
		TWeakInterfacePtr<INNERuntimeRDG> Runtime = UE::NNE::GetRuntime<INNERuntimeRDG>(RuntimeName);
		if (!Runtime.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model instance. No RDG runtime '%ls' found. Valid RDG runtimes are: ", *RuntimeName);
			for (const FString& Name : UE::NNE::GetAllRuntimeNames<INNERuntimeRDG>())
			{
				UE_LOGF(LogNNEDenoiser, Log, "- %ls", *Name);
			}
			return {};
		}

		if (Runtime->CanCreateModelRDG(&ModelData) != INNERuntimeRDG::ECanCreateModelRDGStatus::Ok)
		{
			UE_LOGF(LogNNEDenoiser, Log, "%ls on RDG can not create model", *RuntimeName);
			return {};
		}

		TSharedPtr<UE::NNE::IModelRDG> Model = Runtime->CreateModelRDG(&ModelData);
		if (!Model.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model using %ls on RDG", *RuntimeName);
			return {};
		}

		TSharedPtr<UE::NNE::IModelInstanceRDG> ModelInstance = Model->CreateModelInstanceRDG();
		if (!ModelInstance.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Log, "Could not create model instance using %ls on RDG", *RuntimeName);
			return {};
		}

		return MakeUnique<FModelInstanceRDG>(ModelInstance.ToSharedRef());
	}

	FModelInstanceRDG::FModelInstanceRDG(TSharedRef<UE::NNE::IModelInstanceRDG> ModelInstance) :
		ModelInstance(ModelInstance)
	{

	}

	FModelInstanceRDG::~FModelInstanceRDG()
	{
		
	}

	TConstArrayView<NNE::FTensorDesc> FModelInstanceRDG::GetInputTensorDescs() const
	{
		return ModelInstance->GetInputTensorDescs();
	}

	TConstArrayView<NNE::FTensorDesc> FModelInstanceRDG::GetOutputTensorDescs() const
	{
		return ModelInstance->GetOutputTensorDescs();
	}

	TConstArrayView<NNE::FTensorShape> FModelInstanceRDG::GetInputTensorShapes() const
	{
		return ModelInstance->GetInputTensorShapes();
	}

	TConstArrayView<NNE::FTensorShape> FModelInstanceRDG::GetOutputTensorShapes() const
	{
		return ModelInstance->GetOutputTensorShapes();
	}

	FModelInstanceRDG::ESetInputTensorShapesStatus FModelInstanceRDG::SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes)
	{
		return ModelInstance->SetInputTensorShapes(InInputShapes);
	}

	FModelInstanceRDG::EEnqueueRDGStatus FModelInstanceRDG::EnqueueRDG(FRDGBuilder &GraphBuilder, TConstArrayView<NNE::FTensorBindingRDG> Inputs, TConstArrayView<NNE::FTensorBindingRDG> Outputs)
	{
		return ModelInstance->EnqueueRDG(GraphBuilder, Inputs, Outputs);
	}

}