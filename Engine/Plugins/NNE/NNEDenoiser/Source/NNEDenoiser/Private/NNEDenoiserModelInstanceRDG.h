// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEDenoiserModelInstance.h"
#include "NNEModelData.h"

namespace UE::NNE
{
	class IModelInstanceRDG;
}

namespace UE::NNEDenoiser::Private
{

class FModelInstanceRDG : public IModelInstance
{
public:
	explicit FModelInstanceRDG(TSharedRef<UE::NNE::IModelInstanceRDG> ModelInstance);

	static TUniquePtr<FModelInstanceRDG> Make(UNNEModelData& ModelData, const FString& RuntimeName);

	virtual ~FModelInstanceRDG();

	virtual TConstArrayView<NNE::FTensorDesc> GetInputTensorDescs() const override;

	virtual TConstArrayView<NNE::FTensorDesc> GetOutputTensorDescs() const override;

	virtual TConstArrayView<NNE::FTensorShape> GetInputTensorShapes() const override;

	virtual TConstArrayView<NNE::FTensorShape> GetOutputTensorShapes() const override;

	virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes) override;

	virtual EEnqueueRDGStatus EnqueueRDG(FRDGBuilder &GraphBuilder, TConstArrayView<NNE::FTensorBindingRDG> Inputs, TConstArrayView<NNE::FTensorBindingRDG> Outputs) override;

private:
	TSharedRef<UE::NNE::IModelInstanceRDG> ModelInstance;
};
	
}