// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::NNEDenoiser::Private
{
	
	struct FParameters
	{
		struct FTilingConfig
		{
			int32 Alignment = 1;
			int32 Overlap{};
			int32 MaxSize{};
			int32 MinSize = 1;
		};

		FTilingConfig TilingConfig{};
	};

} // namespace UE::NNEDenoiser::Private