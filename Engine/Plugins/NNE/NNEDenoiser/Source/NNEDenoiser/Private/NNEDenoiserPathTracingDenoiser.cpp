// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserPathTracingDenoiser.h"
#include "NNEDenoiserGenericDenoiser.h"
#include "NNEDenoiserHistory.h"
#include "NNEDenoiserLog.h"
#include "SceneView.h"

namespace UE::NNEDenoiser::Private
{

using UE::Renderer::Private::IPathTracingDenoiser;

FPathTracingDenoiser::FPathTracingDenoiser(TUniquePtr<FGenericDenoiser> Denoiser) : Denoiser(MoveTemp(Denoiser))
{

}

void FPathTracingDenoiser::AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const
{
	TUniquePtr<FHistory> History = Denoiser->AddPasses(GraphBuilder, Inputs.ColorTex, Inputs.AlbedoTex, Inputs.NormalTex, Inputs.OutputTex, nullptr, View.GPUMask, nullptr);
	
	// note: can't store history
	check(!History.IsValid()); 
}

} // namespace UE::NNEDenoiser::Private