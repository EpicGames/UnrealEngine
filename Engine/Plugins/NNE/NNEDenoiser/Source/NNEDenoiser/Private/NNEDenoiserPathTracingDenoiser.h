// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingDenoiser.h"
#include "Templates/UniquePtr.h"

namespace UE::NNEDenoiser::Private
{

class FGenericDenoiser;

using UE::Renderer::Private::IPathTracingDenoiser;

class FPathTracingDenoiser final : public IPathTracingDenoiser
{

public:
	explicit FPathTracingDenoiser(TUniquePtr<FGenericDenoiser> Denoiser);

	~FPathTracingDenoiser() = default;

	bool NeedTextureCreateExtraFlags() const override { return true; }

	void AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const override;

private:
	TUniquePtr<FGenericDenoiser> Denoiser;
};

} // namespace UE::NNEDenoiser::Private