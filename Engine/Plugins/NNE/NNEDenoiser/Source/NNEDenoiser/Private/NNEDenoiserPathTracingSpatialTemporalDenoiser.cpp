// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserPathTracingSpatialTemporalDenoiser.h"
#include "NNEDenoiserGenericDenoiser.h"
#include "NNEDenoiserHistory.h"
#include "NNEDenoiserLog.h"
#include "SceneView.h"

namespace UE::NNEDenoiser::Private
{

using UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser;

FPathTracingSpatialTemporalDenoiser::FPathTracingSpatialTemporalDenoiser(TUniquePtr<FGenericDenoiser> Denoiser) : Denoiser(MoveTemp(Denoiser))
{

}

const TCHAR* FPathTracingSpatialTemporalDenoiser::GetDebugName() const
{
	return Denoiser->GetDebugName();
}

IPathTracingSpatialTemporalDenoiser::FOutputs FPathTracingSpatialTemporalDenoiser::AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const
{
	FHistory* PrevHistory = Inputs.PrevHistory.IsValid() ? reinterpret_cast<FHistory*>(Inputs.PrevHistory.GetReference()) : nullptr;
	
	TUniquePtr<FHistory> NewHistory = Denoiser->AddPasses(GraphBuilder, Inputs.ColorTex, Inputs.AlbedoTex, Inputs.NormalTex, Inputs.OutputTex, Inputs.FlowTex, View.GPUMask, PrevHistory);

	FOutputs Outputs;
	Outputs.NewHistory = NewHistory.Release();

	return Outputs;
}

void FPathTracingSpatialTemporalDenoiser::AddMotionVectorPass(FRDGBuilder& GraphBuilder, const FSceneView& View, const FMotionVectorInputs& Inputs) const
{
	unimplemented();
}

} // namespace UE::NNEDenoiser::Private