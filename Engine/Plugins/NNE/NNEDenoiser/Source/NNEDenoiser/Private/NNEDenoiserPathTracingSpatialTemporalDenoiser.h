// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingDenoiser.h"
#include "Templates/UniquePtr.h"

namespace UE::NNEDenoiser::Private
{

class FGenericDenoiser;

using UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser;

class FPathTracingSpatialTemporalDenoiser final : public IPathTracingSpatialTemporalDenoiser
{
public:
	explicit FPathTracingSpatialTemporalDenoiser(TUniquePtr<FGenericDenoiser> Denoiser);

	~FPathTracingSpatialTemporalDenoiser() = default;

	// IPathTracingSpatialTemporalDenoiser interface
	const TCHAR* GetDebugName() const override;

	bool NeedTextureCreateExtraFlags() const override { return true; }

	FOutputs AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const override;

	void AddMotionVectorPass(FRDGBuilder& GraphBuilder, const FSceneView& View, const FMotionVectorInputs& Inputs) const override;

private:
	TUniquePtr<FGenericDenoiser> Denoiser;
};

} // namespace UE::NNEDenoiser::Private