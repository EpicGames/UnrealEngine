// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserResourceManager.h"
#include "Algo/Transform.h"
#include "NNEDenoiserUtils.h"
#include "NNERuntimeRDG.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

namespace UE::NNEDenoiser::Private
{

void FResourceManager::AddTexture(EResourceName Name, FRDGTextureRef Current, int32 NumFrames)
{
	check(!TextureMap.Contains(Name));
	check(Current != FRDGTextureRef{});
	check(NumFrames > 0);

	// Init with current texture at index 0
	TArray<FRDGTextureRef>& Textures = TextureMap.Emplace(Name, TArray<FRDGTextureRef>{ Current });

	// Find render targets in history, register them as external textures to RDG and add starting at index 1
	if (const auto& RenderTargets = ResourceMap.Find(Name); RenderTargets)
	{
		Algo::Transform(*RenderTargets, Textures, [&] (TRefCountPtr<IPooledRenderTarget> RenderTarget)
		{
			return GraphBuilder.RegisterExternalTexture(RenderTarget);
		});
	}

	auto FillMissingTextures = [&] (TArray<FRDGTextureRef>& Textures, int32 Num)
	{
		if (Textures.Num() >= Num)
		{
			return;
		}

		FRDGTextureRef RefTextue = Textures.Last();

		const int32 OldNum = Textures.Num();

		Textures.SetNumUninitialized(Num);
		for (int32 I = OldNum; I < Textures.Num(); I++)
		{
			Textures[I] = RefTextue;
		}
	};

	// Repeat "last" valid texture for frames missing their texture
	FillMissingTextures(Textures, NumFrames);

	// note: Always create intermediate texture for now.
	const bool bNeedIntermediateTexture = true;

#if UE_BUILD_DEBUG
	static TSet<FString> DebugNames;
#endif

	if (bNeedIntermediateTexture)
	{
		check(!IntermediateTextureMap.Contains(Name));

		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Tiling.TileSize, Current->Desc.Format, Current->Desc.ClearValue, Current->Desc.Flags);

		TArray<FRDGTextureRef>& IntermediateTextures = IntermediateTextureMap.Add(Name);
		IntermediateTextures.SetNumUninitialized(NumFrames);

		for (int32 I = 0; I < IntermediateTextures.Num(); I++)
		{
#if UE_BUILD_DEBUG
			const FString ResourceName = UEnum::GetValueAsName(Name).ToString();
			const FString FrameNumberText = FString::FromInt(I);
			const FString TextureDebugName = FString(TEXT("NNEDenoiser.IntermediateTexture_")) + ResourceName + FrameNumberText;

			const TCHAR* DebugNamePtr = nullptr;
			if (!DebugNames.Contains(TextureDebugName))
			{
				DebugNames.Add(TextureDebugName);
			}

			DebugNamePtr = **DebugNames.Find(TextureDebugName);

			IntermediateTextures[I] = GraphBuilder.CreateTexture(Desc, DebugNamePtr);
#else
			IntermediateTextures[I] = GraphBuilder.CreateTexture(Desc, TEXT("NNEDenoiser.IntermediateTexture"));
#endif
		}
	}
}

void FResourceManager::BeginTile(int32 TileIndex)
{
	check(TileIndex >= 0);
	check(TileIndex < Tiling.Tiles.Num());

	CurrentTileIdx = TileIndex;
	const FTile& Tile = Tiling.Tiles[CurrentTileIdx];

	// Copy sub region from original input texture to tile input texture
	FRHICopyTextureInfo CopyInputInfo{};
	CopyInputInfo.SourcePosition = {Tile.Position.X + Tile.InputOffsets.Min.X, Tile.Position.Y + Tile.InputOffsets.Min.Y, 0};
	CopyInputInfo.DestPosition = {Tile.InputOffsets.Min.X, Tile.InputOffsets.Min.Y, 0};
	CopyInputInfo.Size = {Tiling.TileSize.X + Tile.InputOffsets.Width(), Tiling.TileSize.Y + Tile.InputOffsets.Height(), 1};

	// Only clear if we don't overwrite the whole texture
	const bool bNeedClear = Tile.InputOffsets.Width() != 0 || Tile.InputOffsets.Height() != 0;
	const FLinearColor ClearColor = {.0f, 0.f, .0f, .0f};
	const FLinearColor ClearNormal = {-1.0f, -1.0f, -1.0f, -1.0f};

	for (const auto& KeyValue : TextureMap)
	{
		if (!IntermediateTextureMap.Contains(KeyValue.Key))
		{
			continue;
		}

		const TArray<FRDGTextureRef>& Textures = KeyValue.Value;
		const TArray<FRDGTextureRef>& IntermediateTextures = IntermediateTextureMap.FindChecked(KeyValue.Key);
		check(Textures.Num() == IntermediateTextures.Num());

		const int32 StartIdx = KeyValue.Key == EResourceName::Output ? 1 : 0;
		for (int32 I = StartIdx; I < Textures.Num(); I++)
		{
			if (bNeedClear)
			{
				AddClearRenderTargetPass(GraphBuilder, IntermediateTextures[I], KeyValue.Key == EResourceName::Normal ? ClearNormal : ClearColor);
			}
			AddCopyTexturePass(GraphBuilder, Textures[I], IntermediateTextures[I], CopyInputInfo);
		}
	}
}

void FResourceManager::EndTile()
{
	if (!IntermediateTextureMap.Contains(EResourceName::Output))
	{
		return;
	}

	check(CurrentTileIdx >= 0);
	check(CurrentTileIdx < Tiling.Tiles.Num());
	
	const FTile& Tile = Tiling.Tiles[CurrentTileIdx];

	FRHICopyTextureInfo CopyOutputInfo{};
	CopyOutputInfo.SourcePosition = {Tile.OutputOffsets.Min.X, Tile.OutputOffsets.Min.Y, 0};
	CopyOutputInfo.DestPosition = {Tile.Position.X + Tile.OutputOffsets.Min.X, Tile.Position.Y + Tile.OutputOffsets.Min.Y, 0};
	CopyOutputInfo.Size = {Tiling.TileSize.X + Tile.OutputOffsets.Width(), Tiling.TileSize.Y + Tile.OutputOffsets.Height(), 1};

	const TArray<FRDGTextureRef>& Textures = TextureMap.FindChecked(EResourceName::Output);
	const TArray<FRDGTextureRef>& IntermediateTextures = IntermediateTextureMap.FindChecked(EResourceName::Output);
	check(!Textures.IsEmpty());
	check(Textures.Num() == IntermediateTextures.Num());

	AddCopyTexturePass(GraphBuilder, IntermediateTextures[0], Textures[0], CopyOutputInfo);
}

FRDGTextureRef FResourceManager::GetIntermediateTexture(EResourceName Name, int32 FrameIdx) const
{
	const TArray<FRDGTextureRef>& IntermediateTextures = IntermediateTextureMap.FindChecked(Name);

	check(FrameIdx >= 0);
	check(FrameIdx < IntermediateTextures.Num());
	return IntermediateTextures[FrameIdx];
}

TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>> FResourceManager::MakeHistoryResourceMap()
{
	TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>> Result;

	for (const auto& KeyValue : TextureMap)
	{
		EResourceName Name = KeyValue.Key;
		const TArray<FRDGTextureRef>& Textures = KeyValue.Value;
		FRDGTextureRef CurrentTexture = Textures[0];

		const int32 HistoryNumFrames = FMath::Max(Textures.Num() - 1, 0);

		TArray<TRefCountPtr<IPooledRenderTarget>> NewHistoryRenderTargets;
		if (HistoryNumFrames > 1)
		{
			if (const auto* RenderTargets = ResourceMap.Find(Name); RenderTargets)
			{
				NewHistoryRenderTargets.Append(RenderTargets->GetData(), FMath::Min(RenderTargets->Num(), HistoryNumFrames - 1));
			}
		}

		if (HistoryNumFrames)
		{
			GraphBuilder.QueueTextureExtraction(CurrentTexture, &NewHistoryRenderTargets.InsertDefaulted_GetRef(0));
		}

		if (!NewHistoryRenderTargets.IsEmpty())
		{
			Result.Add(Name, MoveTemp(NewHistoryRenderTargets));
		}
	}

	return Result;
}

FRDGTextureRef FResourceManager::GetTexture(EResourceName Name, int32 FrameIdx, bool bBaseTexture) const
{
	const bool bIntermediate = !bBaseTexture && IntermediateTextureMap.Contains(Name);

	const TMap<EResourceName, TArray<FRDGTextureRef>>& Map = bIntermediate ? IntermediateTextureMap : TextureMap;
	check(Map.Contains(Name));

	const TArray<FRDGTextureRef>& Textures = Map.FindChecked(Name);
	check(FrameIdx >= 0);
	check(FrameIdx < Textures.Num());

	if (FrameIdx < 0 || FrameIdx >= Textures.Num())
	{
		return FRDGTextureRef{};
	}

	return Textures[FrameIdx];
}

}