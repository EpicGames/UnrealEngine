// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "NNEDenoiserResourceName.h"
#include "NNEDenoiserTiling.h"
#include "RendererInterface.h"
#include "RenderGraphFwd.h"

namespace UE::NNE
{
	struct FTensorBindingRDG;
}

namespace UE::NNEDenoiser::Private
{

class FResourceManager
{
public:
	FResourceManager(FRDGBuilder& GraphBuilder, FTiling Tiling, TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>> ResourceMap = {}) :
		GraphBuilder(GraphBuilder), Tiling(Tiling), ResourceMap(ResourceMap)
	{

	}

	void AddTexture(EResourceName Name, FRDGTextureRef Current, int32 NumFrames);

	void BeginTile(int32 TileIndex);
	void EndTile();

	FRDGTextureRef GetIntermediateTexture(EResourceName Name, int32 FrameIdx) const;

	FRDGTextureRef GetTexture(EResourceName Name, int32 FrameIdx) const
	{
		return GetTexture(Name, FrameIdx, false);
	}

	TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>> MakeHistoryResourceMap();

private:
	FRDGTextureRef GetTexture(EResourceName Name, int32 FrameIdx, bool bBaseTexture) const;
	
	FRDGBuilder& GraphBuilder;
	FTiling Tiling;
	TMap<EResourceName, TArray<TRefCountPtr<IPooledRenderTarget>>> ResourceMap;

	TMap<EResourceName, TArray<FRDGTextureRef>> TextureMap;
	TMap<EResourceName, TArray<FRDGTextureRef>> IntermediateTextureMap;

	int32 CurrentTileIdx = -1;
};

} // namespace UE::NNEDenoiser::Private