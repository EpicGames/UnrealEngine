// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserResourceMapping.h"

namespace UE::NNEDenoiser::Private
{

namespace Detail
{

FResourceMappingList MakeTensorLayout(const TMap<int32, TMap<int32, FResourceInfo>>& ResourceMap)
{
	FResourceMappingList Result{};

	for (int32 I = 0; I < ResourceMap.Num(); I++)
	{
		checkf(ResourceMap.Contains(I), TEXT("Missing intput/output %d, must be continuous!"), I);

		const TMap<int32, FResourceInfo>& InnerMap = ResourceMap[I];
		FResourceMapping& Mapping = Result.Add_GetRef({});

		for (int32 J = 0; J < InnerMap.Num(); J++)
		{
			checkf(InnerMap.Contains(J), TEXT("Missing tensor info for channel %d, must be continuous!"), J);

			Mapping.Add(InnerMap[J]);
		}
	}

	return Result;
}


void Add(TMap<int32, TMap<int32, FResourceInfo>>& Map, int32 TensorIndex, int32 TensorChannel, EResourceName ResourceName, int32 ResourceChannel, int32 FrameIndex)
{
	if (TensorChannel < 0)
	{
		for (int32 I = 0; I < -TensorChannel; I++)
		{
			Map.FindOrAdd(TensorIndex).FindOrAdd(I) = FResourceInfo{ResourceName, I, 0};
		}
	}
	else
	{
		Map.FindOrAdd(TensorIndex).FindOrAdd(TensorChannel) = FResourceInfo{ResourceName, ResourceChannel, 0};
	}
}

} // namespace Detail

void FResourceMapping::Add(FResourceInfo Info)
{
	ChannelMapping.Add(MoveTemp(Info));
}

FResourceInfo& FResourceMapping::Add_GetRef(FResourceInfo Info)
{
	return ChannelMapping.Add_GetRef(MoveTemp(Info));
}

const FResourceInfo& FResourceMapping::GetChecked(int32 Channel) const
{
	check(Channel >= 0 && Channel < ChannelMapping.Num());

	return ChannelMapping[Channel];
}

int32 FResourceMapping::Num() const
{
	return ChannelMapping.Num();
}

bool FResourceMapping::HasResource(EResourceName Name) const
{
	for (const auto& Info: ChannelMapping)
	{
		if (Info.Name == Name)
		{
			return true;
		}
	}
	return false;
}

int32 FResourceMapping::NumFrames(EResourceName Name) const
{
	int32 MinFrameIndex = 1;
	for (const auto& Info: ChannelMapping)
	{
		if (Info.Name == Name)
		{
			MinFrameIndex = FMath::Min(MinFrameIndex, Info.Frame);
		}
	}

	return 1 - MinFrameIndex;
}

TMap<int32, TArray<FChannelMapping>> FResourceMapping::GetChannelMappingPerFrame(EResourceName Name) const
{
	TMap<int32, TArray<FChannelMapping>> Result;
	for (int32 I = 0; I < ChannelMapping.Num(); I++)
	{
		const FResourceInfo& Info = ChannelMapping[I];

		if (Info.Name == Name)
		{
			Result.FindOrAdd(Info.Frame).Add({I, Info.Channel});
		}
	}

	return Result;
}

void FResourceMappingList::Add(FResourceMapping ResourceMapping)
{
	InputMapping.Add(MoveTemp(ResourceMapping));
}

FResourceMapping& FResourceMappingList::Add_GetRef(FResourceMapping ResourceMapping)
{
	return InputMapping.Add_GetRef(MoveTemp(ResourceMapping));
}

const FResourceMapping* FResourceMappingList::Get(int32 InputIndex) const
{
	if (InputIndex >= 0 && InputIndex < InputMapping.Num())
	{
		return &InputMapping[InputIndex];
	}

	return {};
}

const FResourceMapping& FResourceMappingList::GetChecked(int32 InputIndex) const
{
	check(InputIndex >= 0 && InputIndex < InputMapping.Num());

	return InputMapping[InputIndex];
}

int32 FResourceMappingList::Num() const
{
	return InputMapping.Num();
}

int32 FResourceMappingList::NumChannels(int32 InputIndex) const
{
	check(InputIndex >= 0 && InputIndex < InputMapping.Num());
	
	return InputMapping[InputIndex].Num();
}

int32 FResourceMappingList::NumFrames(EResourceName Name) const
{
	int32 MaxNumFrames = 0;
	for (const auto& Input : InputMapping)
	{
		const int32 NumFrames = Input.NumFrames(Name);
		if (MaxNumFrames < NumFrames)
		{
			MaxNumFrames = NumFrames;
		}
	}

	return MaxNumFrames;
}

bool FResourceMappingList::HasResource(EResourceName Name) const
{
	for (const auto& Input : InputMapping)
	{
		if (Input.HasResource(Name))
		{
			return true;
		}
	}

	return false;
}

bool FResourceMappingList::HasResource(int32 InputIndex, EResourceName Name) const
{
	check(InputIndex >= 0 && InputIndex < InputMapping.Num());

	return InputMapping[InputIndex].HasResource(Name);
}

TMap<int32, TArray<FChannelMapping>> FResourceMappingList::GetChannelMappingPerFrame(int32 InputIndex, EResourceName Name) const
{
	check(InputIndex >= 0 && InputIndex < InputMapping.Num());

	return InputMapping[InputIndex].GetChannelMappingPerFrame(Name);
}

template<>
FResourceMappingList MakeTensorLayout<FNNEDenoiserInputMappingData>(UDataTable* DataTable)
{
	TMap<int32, TMap<int32, FResourceInfo>> Map;
	DataTable->ForeachRow<FNNEDenoiserInputMappingData>("FResourceLayout", [&] (const FName &Key, const FNNEDenoiserInputMappingData &Value)
	{
		Detail::Add(Map, Value.TensorIndex, Value.TensorChannel, ToResourceName(Value.Resource), Value.ResourceChannel, 0);
	});

	return Detail::MakeTensorLayout(Map);
}

template<>
FResourceMappingList MakeTensorLayout<FNNEDenoiserOutputMappingData>(UDataTable* DataTable)
{
	TMap<int32, TMap<int32, FResourceInfo>> Map;
	DataTable->ForeachRow<FNNEDenoiserOutputMappingData>("FResourceLayout", [&] (const FName &Key, const FNNEDenoiserOutputMappingData &Value)
	{
		Detail::Add(Map, Value.TensorIndex, Value.TensorChannel, ToResourceName(Value.Resource), Value.ResourceChannel, 0);
	});

	return Detail::MakeTensorLayout(Map);
}

template<>
FResourceMappingList MakeTensorLayout<FNNEDenoiserTemporalInputMappingData>(UDataTable* DataTable)
{
	TMap<int32, TMap<int32, FResourceInfo>> Map;
	DataTable->ForeachRow<FNNEDenoiserTemporalInputMappingData>("FResourceLayout", [&] (const FName &Key, const FNNEDenoiserTemporalInputMappingData &Value)
	{
		Detail::Add(Map, Value.TensorIndex, Value.TensorChannel, ToResourceName(Value.Resource), Value.ResourceChannel, Value.FrameIndex);
	});

	return Detail::MakeTensorLayout(Map);
}

template<>
FResourceMappingList MakeTensorLayout<FNNEDenoiserTemporalOutputMappingData>(UDataTable* DataTable)
{
	TMap<int32, TMap<int32, FResourceInfo>> Map;
	DataTable->ForeachRow<FNNEDenoiserTemporalOutputMappingData>("FResourceLayout", [&] (const FName &Key, const FNNEDenoiserTemporalOutputMappingData &Value)
	{
		Detail::Add(Map, Value.TensorIndex, Value.TensorChannel, ToResourceName(Value.Resource), Value.ResourceChannel, 0);
	});

	return Detail::MakeTensorLayout(Map);
}

} // namespace UE::NNEDenoiser::Private