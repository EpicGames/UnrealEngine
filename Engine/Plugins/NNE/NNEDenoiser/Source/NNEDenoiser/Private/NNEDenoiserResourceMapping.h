// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "NNEDenoiserIOMappingData.h"
#include "NNEDenoiserResourceName.h"

namespace UE::NNEDenoiser::Private
{

struct FResourceInfo
{
	EResourceName Name;
	int32 Channel;
	int32 Frame;
};

struct FChannelMapping
{
	int32 TensorChannel;
	int32 ResourceChannel;
};

// Resource mapping for one tensor:
// 0: ResourceInfo0 -> tensor channel 0 maps to channel ResourceInfo0.Channel of resource ResourceInfo0.Name from frame ResourceInfo0.Frame
// 1: ...
class FResourceMapping
{
public:
	// Add mapping from tensor channel Num()-1 to resource defined by 'Info'
	void Add(FResourceInfo Info);

	// Add mapping from tensor channel Num()-1 to resource defined by 'Info' and return reference to it
	FResourceInfo& Add_GetRef(FResourceInfo Info);

	const FResourceInfo& GetChecked(int32 Channel) const;

	// Number of mapped tensor channels
	int32 Num() const;

	// Number of past frames affected by the mapping including current frame for resource 'Name'
	int32 NumFrames(EResourceName Name) const;

	// Does mapping use resource 'Name'
	bool HasResource(EResourceName Name) const;

	// Get mapping from tensor channel to resource channel per frame for resource 'Name'
	TMap<int32, TArray<FChannelMapping>> GetChannelMappingPerFrame(EResourceName Name) const;

private:
	TArray<FResourceInfo> ChannelMapping;
};

// Resource mapping list for multiple tensors:
// 0: ResourceMapping0 -> tensor 0 maps to resources as defined by ResourceMapping0
// 1: ...
class FResourceMappingList
{
public:
	// Add resource mapping for tensor Num()-1
	void Add(FResourceMapping ResourceMapping);

	// Add resource mapping for tensor Num()-1 and return reference to it
	FResourceMapping& Add_GetRef(FResourceMapping ResourceMapping);

	const FResourceMapping* Get(int32 InputIndex) const;
	const FResourceMapping& GetChecked(int32 InputIndex) const;

	// Number of mapped tensors
	int32 Num() const;

	// Number of mapped tensor channels for input tensor 'InputIndex'
	int32 NumChannels(int32 InputIndex) const;

	// Number of past frames affected by the mapping including current frame for resource 'Name'
	int32 NumFrames(EResourceName Name) const;

	// Does mapping use resource 'Name'
	bool HasResource(EResourceName Name) const;

	// Does mapping for tensor 'InputIndex' use resource 'Name'
	bool HasResource(int32 InputIndex, EResourceName Name) const;

	// Get mapping from tensor channel to resource channel per frame for resource 'Name' and tensor 'InputIndex"
	TMap<int32, TArray<FChannelMapping>> GetChannelMappingPerFrame(int32 InputIndex, EResourceName Name) const;

private:
	TArray<FResourceMapping> InputMapping;
};

// Helper method to make resource mapping list from data table asset using row struct 'RowStructType'
template<class RowStructType>
FResourceMappingList MakeTensorLayout(UDataTable* DataTable);

} // namespace UE::NNEDenoiser::Private