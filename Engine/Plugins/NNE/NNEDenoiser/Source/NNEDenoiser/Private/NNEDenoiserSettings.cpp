// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserSettings.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NNEDenoiserSettings)

UNNEDenoiserSettings::UNNEDenoiserSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName = TEXT("NNE Denoiser");

	DenoiserAsset = FSoftObjectPath(TEXT("/NNEDenoiser/NNED_Oidn2-3_Balanced_Alpha.NNED_Oidn2-3_Balanced_Alpha"));
	TemporalDenoiserAsset = FSoftObjectPath(TEXT("/NNEDenoiser/NNEDT_Oidn2-3_Balanced_Alpha.NNEDT_Oidn2-3_Balanced_Alpha"));
}

void UNNEDenoiserSettings::PostInitProperties()
{
	if (IsTemplate())
	{
		// We want the .ini file to have precedence over the CVar constructor, so we apply the ini to the CVar before following the regular UDeveloperSettingsBackedByCVars flow
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/NNEDenoiser.NNEDenoiserSettings"), *GEngineIni, ECVF_SetByProjectSetting);
	}

	Super::PostInitProperties();
}