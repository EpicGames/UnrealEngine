// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/IntVector.h"
#include "Math/MathFwd.h"
#include "NNEDenoiserUtils.h"

namespace UE::NNEDenoiser::Private
{
	
	struct FTile
	{
		FTile(FIntPoint Position, FIntRect InputOffsets, FIntRect OutputOffsets) : Position(Position), InputOffsets(InputOffsets), OutputOffsets(OutputOffsets) {}
		
		FIntPoint Position;
		FIntRect InputOffsets;
		FIntRect OutputOffsets;
	};

	struct FTiling
	{
		FTiling() = default;

		FIntPoint TileSize;
		FIntPoint Count;
		TArray<FTile> Tiles;
	};

	inline FTiling CreateTiling(FIntPoint TargetTileSize, FIntPoint MaxTileSize, FIntPoint MinTileSize, int32 TileAlignment, FIntPoint MinimumOverlap, FIntPoint Size)
	{
		auto GetTileSize = [] (int32 Max, int32 Min, int32 Alignment, int32 Overlap, int32 Size)
		{
			int NumTiles = 1;
			int32 Result = FMath::Max(RoundUp(Size, Alignment), Min);

			if (Max <= 0)
			{
				return Result;
			}

			Alignment = FMath::Max(Alignment, 1);
			Min = FMath::Max(Min, Alignment + Overlap);

			while (Result > Max && Result > Min)
			{
				NumTiles++;
				Result = FMath::Max(RoundUp(CeilDiv(Size - Overlap, NumTiles), Alignment) + Overlap, Min);
			}

			return Result;
		};

		auto GetNumTiles = [] (int32 TileSize, int32 Overlap, int32 Size)
		{
			return Size > TileSize ? CeilDiv(Size - Overlap, TileSize - Overlap) : 1;
		};

		auto GetOffsets = [] (int32 Count, int32 TileSize, int32 Overlap, int32 Size)
		{
			TArray<int32> Result;
			for (int32 I = 0; I < Count; I++)
			{
				Result.Add(I < Count - 1 ? I * (TileSize - Overlap) : FMath::Max(Size - TileSize, 0));
			}

			return Result;
		};

		FTiling Result{};
		Result.TileSize = {
			TargetTileSize.X > 0 ? TargetTileSize.X : GetTileSize(MaxTileSize.X, MinTileSize.X, TileAlignment, MinimumOverlap.X, Size.X),
			TargetTileSize.Y > 0 ? TargetTileSize.Y : GetTileSize(MaxTileSize.Y, MinTileSize.Y, TileAlignment, MinimumOverlap.Y, Size.Y)
		};

		Result.Count = {
			GetNumTiles(Result.TileSize.X, MinimumOverlap.X, Size.X),
			GetNumTiles(Result.TileSize.Y, MinimumOverlap.Y, Size.Y)
		};

		const FIntPoint TotalOverlap = Result.Count * Result.TileSize - Size;
		const FIntPoint Overlap = {
			Result.Count.X == 1 ? 0 : TotalOverlap.X / (Result.Count.X - 1),
			Result.Count.Y == 1 ? 0 : TotalOverlap.Y / (Result.Count.Y - 1)
		};

		const FIntPoint HalfOverlap = {
			FMath::FloorToInt32(Overlap.X / 2.0f),
			FMath::FloorToInt32(Overlap.Y / 2.0f)
		};

		const TArray<int32> OffsetsX = GetOffsets(Result.Count.X, Result.TileSize.X, Overlap.X, Size.X);
		const TArray<int32> OffsetsY = GetOffsets(Result.Count.Y, Result.TileSize.Y, Overlap.Y, Size.Y);

		const int32 InX0 = 0;
		const int32 InX1 = -FMath::Max(Result.TileSize.X - Size.X, 0);
		const int32 InY0 = 0;
		const int32 InY1 = -FMath::Max(Result.TileSize.Y - Size.Y, 0);

		for (int32 Ty = 0; Ty < Result.Count.Y; Ty++)
		{
			const int32 Y0 = OffsetsY[Ty];
			const int32 Y1 = Y0 + Result.TileSize.Y;

			const int32 OutY0 = Result.Count.Y == 1 ? InY0 : Ty > 0 ? HalfOverlap.Y : 0;
			const int32 OutY1 = Result.Count.Y == 1 ? InY1 : Ty < Result.Count.Y - 1 ? -HalfOverlap.Y : 0;

			for (int32 Tx = 0; Tx < Result.Count.X; Tx++)
			{
				const int32 X0 = OffsetsX[Tx];
				const int32 X1 = X0 + Result.TileSize.X;

				const int32 OutX0 = Result.Count.X == 1 ? InX0 : Tx > 0 ? HalfOverlap.X : 0;
				const int32 OutX1 = Result.Count.X == 1 ? InX1 : Tx < Result.Count.X - 1 ? -HalfOverlap.X : 0;

				Result.Tiles.Add(FTile{{X0, Y0}, {InX0, InY0, InX1, InY1}, {OutX0, OutY0, OutX1, OutY1}});
			}
		}

		return Result;
	}

}