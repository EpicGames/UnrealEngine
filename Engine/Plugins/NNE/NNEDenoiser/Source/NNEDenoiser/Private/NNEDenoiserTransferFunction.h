// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "RenderGraphFwd.h"

struct FLinearColor;

namespace UE::NNEDenoiser::Private
{

class ITransferFunction
{
public:
	virtual ~ITransferFunction() = default;

	virtual void Forward(TConstArrayView<FLinearColor> InputImage, float InputScale, TArray<FLinearColor>& OutputImage) const = 0;
	virtual void Inverse(TConstArrayView<FLinearColor> InputImage, float InvInputScale, TArray<FLinearColor>& OutputImage) const = 0;

	virtual void RDGSetInputScale(FRDGBufferRef InputScaleBuffer) = 0;
	
	virtual void RDGForward(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGTextureRef OutputTexture) const = 0;
	virtual void RDGInverse(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGTextureRef OutputTexture) const = 0;
};

}
