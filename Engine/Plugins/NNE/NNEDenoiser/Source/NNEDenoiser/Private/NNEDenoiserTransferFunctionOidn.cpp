// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserTransferFunctionOidn.h"
#include "NNEDenoiserShadersTransferFunctionOidnCS.h"
#include "Algo/Transform.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_GPU_STAT_NAMED(FNNEDenoiserTransferFunctionOidn, TEXT("NNEDenoiser.TransferFunctionOidn"));

namespace UE::NNEDenoiser::Private::Oidn
{

namespace Helper
{

constexpr float HDRMax = 65504.f; // maximum HDR value

// from https://github.com/OpenImageDenoise/oidn
struct FPU
{
	static constexpr float a  =  1.41283765e+03f;
	static constexpr float b  =  1.64593172e+00f;
	static constexpr float c  =  4.31384981e-01f;
	static constexpr float d  = -2.94139609e-03f;
	static constexpr float e  =  1.92653254e-01f;
	static constexpr float f  =  6.26026094e-03f;
	static constexpr float g  =  9.98620152e-01f;
	static constexpr float y0 =  1.57945760e-06f;
	static constexpr float y1 =  3.22087631e-02f;
	static constexpr float x0 =  2.23151711e-03f;
	static constexpr float x1 =  3.70974749e-01f;

	static float Forward(float y)
	{
		if (y <= y0)
		return a * y;
		else if (y <= y1)
		return b * FMath::Pow(y, c) + d;
		else
		return e * FMath::Loge(y + f) + g;
	}

	static float Inverse(float x)
	{
		if (x <= x0)
		return x / a;
		else if (x <= x1)
		return FMath::Pow((x - d) / b, 1.f/c);
		else
		return FMath::Exp((x - g) / e) - f;
	}
};

void ApplyTransferFunction(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	FRDGBufferRef InputScaleBuffer,
	float NormScale,
	float InvNormScale,
	bool Forward)
{
	using namespace UE::NNEDenoiserShaders::Internal;

	const FIntVector Size = InputTexture->Desc.GetSize();
	check(Size == OutputTexture->Desc.GetSize());

	FTransferFunctionOidnCS::FParameters *ShaderParameters = GraphBuilder.AllocParameters<FTransferFunctionOidnCS::FParameters>();
	ShaderParameters->Width = Size.X;
	ShaderParameters->Height = Size.Y;
	ShaderParameters->InputTexture = InputTexture;
	ShaderParameters->InputScaleBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputScaleBuffer, EPixelFormat::PF_R32_FLOAT));
	ShaderParameters->NormScale = NormScale;
	ShaderParameters->InvNormScale = InvNormScale;
	ShaderParameters->OutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutputTexture));

	FTransferFunctionOidnCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FTransferFunctionOidnCS::FTransferFunctionOidnMode>(Forward ? ETransferFunctionOidnMode::Forwward : ETransferFunctionOidnMode::Inverse);

	FIntVector ThreadGroupCount = FIntVector(
		FMath::DivideAndRoundUp(Size.X, FTransferFunctionOidnConstants::THREAD_GROUP_SIZE),
		FMath::DivideAndRoundUp(Size.Y, FTransferFunctionOidnConstants::THREAD_GROUP_SIZE),
		1);

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FTransferFunctionOidnCS> Shader(GlobalShaderMap, PermutationVector);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEDenoiserTransferFunctionOidn, "NNEDenoiser.TransferFunctionOidn");

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("NNEDenoiser.TransferFunctionOidn"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		Shader,
		ShaderParameters,
		ThreadGroupCount);
}

} // Helpser

FTransferFunction::FTransferFunction()
{
	InvNormScale = Helper::FPU::Forward(Helper::HDRMax);
	NormScale = 1.0f / InvNormScale;
}

void FTransferFunction::Forward(TConstArrayView<FLinearColor> InputImage, float InputScale, TArray<FLinearColor>& OutputImage) const
{
	auto InputProcess = [this, InputScale] (float Value)
	{
		Value *= InputScale;
		Value = FMath::Clamp(Value, 0.0f, FLT_MAX);
		
		return Helper::FPU::Forward(Value) * NormScale;
	};

	Algo::Transform(InputImage, OutputImage, [&InputProcess] (const FLinearColor& Color)
	{
		return FLinearColor(
			InputProcess(Color.R),
			InputProcess(Color.G),
			InputProcess(Color.B),
			Color.A
		);
	});
}

void FTransferFunction::Inverse(TConstArrayView<FLinearColor> InputImage, float InvInputScale, TArray<FLinearColor>& OutputImage) const
{
	auto OutputProcess = [this, InvInputScale] (float Value)
	{
		Value = FMath::Clamp(Value, 0.0f, FLT_MAX);
		Value = Helper::FPU::Inverse(Value * InvNormScale);
		Value *= InvInputScale;

		return Value;
	};

	Algo::Transform(InputImage, OutputImage, [&OutputProcess] (const FLinearColor& Color)
	{
		return FLinearColor(
			OutputProcess(Color.R),
			OutputProcess(Color.G),
			OutputProcess(Color.B),
			Color.A
		);
	});
}

void FTransferFunction::RDGSetInputScale(FRDGBufferRef InInputScaleBuffer)
{
	check(InInputScaleBuffer);

	InputScaleBuffer = InInputScaleBuffer;
}

void FTransferFunction::RDGForward(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGTextureRef OutputTexture) const
{
	check(InputScaleBuffer);

	Helper::ApplyTransferFunction(GraphBuilder, InputTexture, OutputTexture, InputScaleBuffer, NormScale, InvNormScale, true);
}

void FTransferFunction::RDGInverse(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGTextureRef OutputTexture) const
{
	check(InputScaleBuffer);
	
	Helper::ApplyTransferFunction(GraphBuilder, InputTexture, OutputTexture, InputScaleBuffer, NormScale, InvNormScale, false);
}

} // namespace UE::NNEDenoiser::Private::Oidn