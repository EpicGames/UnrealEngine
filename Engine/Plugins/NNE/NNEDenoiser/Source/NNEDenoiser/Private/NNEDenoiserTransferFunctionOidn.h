// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEDenoiserTransferFunction.h"

namespace UE::NNEDenoiser::Private::Oidn
{

class FTransferFunction : public ITransferFunction
{
public:
	FTransferFunction();
	virtual ~FTransferFunction() = default;

	virtual void Forward(TConstArrayView<FLinearColor> InputImage, float InputScale, TArray<FLinearColor>& OutputImage) const override;
	virtual void Inverse(TConstArrayView<FLinearColor> InputImage, float InvInputScale, TArray<FLinearColor>& OutputImage) const override;

	virtual void RDGSetInputScale(FRDGBufferRef InInputScaleBuffer) override;
	
	virtual void RDGForward(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGTextureRef OutputTexture) const override;
	virtual void RDGInverse(FRDGBuilder& GraphBuilder, FRDGTextureRef InputTexture, FRDGTextureRef OutputTexture) const override;

private:
	float NormScale = 1.0f;
	float InvNormScale = 1.0f;
	FRDGBufferRef InputScaleBuffer{};
};

} // UE::NNEDenoiser::Private::Oidn
