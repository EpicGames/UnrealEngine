// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "NNEDenoiserTransferFunctionOidn.h"
#include "NNEDenoiserUtils.h"
#include "NNERuntimeCPU.h"
#include "NNERuntimeRDG.h"
#include "RenderGraphBuilder.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::NNEDenoiser::Private
{

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_SHADER_PARAMETER_STRUCT(FTransferFunctionTestUploadTextureParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FTransferFunctionTestUploadBufferParameters, )
	RDG_BUFFER_ACCESS(InputBuffer, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FTransferFunctionTestDownloadParameters, )
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTransferFunctionTest, "NNEDenoiser.UnitTests.OIDN.TransferFunction.Forward", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags_FeatureMask | EAutomationTestFlags::EngineFilter)
bool FTransferFunctionTest::RunTest(const FString& Parameter)
{
	const int32 Width = 1920;
	const int32 Height = 1088;
	const int32 Num = Width * Height;

	const TArray<float> InputScale = {0.5f, 2.0f};

	TArray<FLinearColor> TestData;
	TestData.SetNumUninitialized(Num);

	for (int i = 0; i < TestData.Num(); i++)
	{
		TestData[i] = FLinearColor(FMath::FRand(), FMath::FRand(), FMath::FRand(), 1.0);
	}

	Oidn::FTransferFunction TransferFunction;

	TArray<FLinearColor> ResultCPU;
	TransferFunction.Forward(TestData, InputScale[0], ResultCPU);

	TArray<FLinearColor> ResultRDG;
	ResultRDG.SetNumZeroed(Num);

	FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool(true);

	ENQUEUE_RENDER_COMMAND(NNETransferFunctionTest)
	(
		[&TestData, &ResultRDG, &InputScale, TransferFunction, Num, Signal](FRHICommandListImmediate& RHICmdList) mutable
		{
			ERHIPipeline Pipeline = RHICmdList.GetPipeline();
			if (Pipeline == ERHIPipeline::None)
			{
				RHICmdList.SwitchPipeline(ERHIPipeline::Graphics);
			}

			FRDGBuilder	GraphBuilder(RHICmdList);

			const FRDGTextureDesc InputTextureDesc = FRDGTextureDesc::Create2D({Width, Height}, EPixelFormat::PF_A32B32G32R32F, FClearValueBinding::Black, ETextureCreateFlags::None);
			FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(InputTextureDesc, TEXT("TransferFunctionInputTexture"));

			{
				FTransferFunctionTestUploadTextureParameters* Parameters = GraphBuilder.AllocParameters<FTransferFunctionTestUploadTextureParameters>();
				Parameters->InputTexture = InputTexture;

				GraphBuilder.AddPass(RDG_EVENT_NAME("NNEDenoiser.TransferFunctionTest.UploadInput"), Parameters, ERDGPassFlags::Readback,
				[&TestData, Parameters](FRHICommandListImmediate& RHICmdList)
				{
					SCOPED_NAMED_EVENT_TEXT("FTransferFunctionTest::UploadInput", FColor::Magenta);

					CopyTextureFromCPUToGPU(RHICmdList, TestData, {Width, Height}, Parameters->InputTexture->GetRHI());
				});
			}

			FRDGBufferDesc InputScaleBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), 2);
			FRDGBufferRef InputScaleBuffer = GraphBuilder.CreateBuffer(InputScaleBufferDesc, TEXT("TransferFunctionInputScaleBuffer"));

			{
				FTransferFunctionTestUploadBufferParameters* Parameters = GraphBuilder.AllocParameters<FTransferFunctionTestUploadBufferParameters>();
				Parameters->InputBuffer = InputScaleBuffer;

				GraphBuilder.AddPass(RDG_EVENT_NAME("NNEDenoiser.TransferFunctionTest.UploadScale"), Parameters, ERDGPassFlags::Readback,
				[&InputScale, Parameters](FRHICommandListImmediate& RHICmdList)
				{
					SCOPED_NAMED_EVENT_TEXT("FTransferFunctionTest::UploadScale", FColor::Magenta);

					CopyBufferFromCPUToGPU(RHICmdList, InputScale, 2, Parameters->InputBuffer->GetRHI());
				});
			}

			const FRDGTextureDesc OutputTextureDesc = FRDGTextureDesc::Create2D({Width, Height}, EPixelFormat::PF_A32B32G32R32F, FClearValueBinding::Black, ETextureCreateFlags::UAV);
			FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputTextureDesc, TEXT("TransferFunctionOutputTexture"));

			TransferFunction.RDGSetInputScale(InputScaleBuffer);

			TransferFunction.RDGForward(GraphBuilder, InputTexture, OutputTexture);

			{
				FTransferFunctionTestDownloadParameters* Parameters = GraphBuilder.AllocParameters<FTransferFunctionTestDownloadParameters>();
				Parameters->OutputTexture = OutputTexture;

				GraphBuilder.AddPass(RDG_EVENT_NAME("NNEDenoiser.TransferFunctionTest.Download"), Parameters, ERDGPassFlags::Readback,
				[&ResultRDG, Num, Parameters](FRHICommandListImmediate& RHICmdList)
				{
					SCOPED_NAMED_EVENT_TEXT("FTransferFunctionTest::Download", FColor::Magenta);

					CopyTextureFromGPUToCPU(RHICmdList, Parameters->OutputTexture->GetRHI(), {Width, Height}, ResultRDG);
				});
			}

			GraphBuilder.Execute();

			RHICmdList.SubmitAndBlockUntilGPUIdle();

			Signal->Trigger();
		}
	);

	Signal->Wait();

	for (int32 i = 0; i < Num; i++)
	{
		UTEST_EQUAL_TOLERANCE(TEXT("TransferFunction"), ResultRDG[i].R, ResultCPU[i].R, 1e-6f);
		UTEST_EQUAL_TOLERANCE(TEXT("TransferFunction"), ResultRDG[i].G, ResultCPU[i].G, 1e-6f);
		UTEST_EQUAL_TOLERANCE(TEXT("TransferFunction"), ResultRDG[i].B, ResultCPU[i].B, 1e-6f);
		UTEST_EQUAL(TEXT("TransferFunction"), ResultRDG[i].A, ResultCPU[i].A);
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNEDenoiser::Internal
