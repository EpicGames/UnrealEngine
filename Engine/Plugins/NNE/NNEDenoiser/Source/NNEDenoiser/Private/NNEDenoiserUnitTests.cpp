// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "NNEDenoiserAutoExposure.h"
#include "NNEDenoiserUtils.h"
#include "NNERuntimeCPU.h"
#include "NNERuntimeRDG.h"
#include "RenderGraphBuilder.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::NNEDenoiser::Private
{

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_SHADER_PARAMETER_STRUCT(FAutoExposureTestUploadParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FAutoExposureTestDownloadParameters, )
	RDG_BUFFER_ACCESS(OutputBuffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoExposureTest, "NNEDenoiser.UnitTests.AutoExposure", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags_FeatureMask | EAutomationTestFlags::EngineFilter)
bool FAutoExposureTest::RunTest(const FString& Parameter)
{
	const int32 Width = 1920;
	const int32 Height = 1088;

	TArray<FLinearColor> TestData;
	TestData.SetNumUninitialized(Width * Height);

	for (int i = 0; i < TestData.Num(); i++)
	{
		TestData[i] = FLinearColor(FMath::FRand(), FMath::FRand(), FMath::FRand(), 1.0);
	}

	FAutoExposure AutoExposure;

	float ResultCPU = 0.0f;
	AutoExposure.Run(TestData, {Width, Height}, ResultCPU);

	TArray<float> ResultRDG;
	ResultRDG.SetNumZeroed(2);

	FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool(true);

	ENQUEUE_RENDER_COMMAND(NNEAutoExposureTest)
	(
		[&TestData, &ResultRDG, AutoExposure, Signal](FRHICommandListImmediate& RHICmdList) mutable
		{
			ERHIPipeline Pipeline = RHICmdList.GetPipeline();
			if (Pipeline == ERHIPipeline::None)
			{
				RHICmdList.SwitchPipeline(ERHIPipeline::Graphics);
			}

			FRDGBuilder	GraphBuilder(RHICmdList);

			const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D({Width, Height}, EPixelFormat::PF_A32B32G32R32F, FClearValueBinding::Black, ETextureCreateFlags::None);
			FRDGTextureRef InputTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("AutoExposureInputTexture"));

			{
				FAutoExposureTestUploadParameters* Parameters = GraphBuilder.AllocParameters<FAutoExposureTestUploadParameters>();
				Parameters->InputTexture = InputTexture;

				GraphBuilder.AddPass(RDG_EVENT_NAME("NNEDenoiser.AutoExposureTest.Upload"), Parameters, ERDGPassFlags::Readback,
				[&TestData, Parameters](FRHICommandListImmediate& RHICmdList)
				{
					SCOPED_NAMED_EVENT_TEXT("FAutoExposureTest::Upload", FColor::Magenta);

					CopyTextureFromCPUToGPU(RHICmdList, TestData, {Width, Height}, Parameters->InputTexture->GetRHI());
				});
			}

			FRDGBufferDesc InputBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), 2);
			FRDGBufferRef OutputBuffer = GraphBuilder.CreateBuffer(InputBufferDesc, TEXT("AutoExposureOutputBuffer"));

			AutoExposure.EnqueueRDG(GraphBuilder, InputTexture, OutputBuffer);

			{
				FAutoExposureTestDownloadParameters* Parameters = GraphBuilder.AllocParameters<FAutoExposureTestDownloadParameters>();
				Parameters->OutputBuffer = OutputBuffer;

				GraphBuilder.AddPass(RDG_EVENT_NAME("NNEDenoiser.AutoExposureTest.Download"), Parameters, ERDGPassFlags::Readback,
				[&ResultRDG, Parameters](FRHICommandListImmediate& RHICmdList)
				{
					SCOPED_NAMED_EVENT_TEXT("FAutoExposureTest::Download", FColor::Magenta);

					CopyBufferFromGPUToCPU(RHICmdList, Parameters->OutputBuffer->GetRHI(), 2, ResultRDG);
				});
			}

			GraphBuilder.Execute();

			RHICmdList.SubmitAndBlockUntilGPUIdle();

			Signal->Trigger();
		}
	);

	Signal->Wait();

	UTEST_EQUAL(TEXT("AutoExposure"), ResultRDG[0], ResultCPU);

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNEDenoiser::Internal
