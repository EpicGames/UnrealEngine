// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEDenoiserLog.h"
#include "NNEDenoiserParameters.h"
#include "NNEDenoiserShadersMappedCopyCS.h"
#include "NNETypes.h"
#include "RHICommandList.h"
#include "RHIGPUReadback.h"
#include "RHIResources.h"
#include "RHITypes.h"

namespace UE::NNEDenoiser::Private
{
	// Returns ceil(a / b) for non-negative integers
	template<typename Int, typename IntB>
	Int CeilDiv(Int a, IntB b)
	{
		return (a + b - 1) / b;
	}

	// Returns a rounded up to multiple of b
	template<typename Int, typename IntB>
	Int RoundUp(Int a, IntB b)
	{
		return CeilDiv(a, b) * b;
	}

	template<class IntType>
	bool IsTensorShapeValid(TConstArrayView<IntType> ShapeData, TConstArrayView<int32> RequiredShapeData, const FString& Label)
	{
		static_assert(std::is_integral_v<IntType>);

		if (ShapeData.Num() != RequiredShapeData.Num())
		{
			UE_LOGF(LogNNEDenoiser, Error, "%ls has wrong rank (expected %d, got %d)!", *Label, RequiredShapeData.Num(), ShapeData.Num())
			return false;
		}

		for (int32 I = 0; I < RequiredShapeData.Num(); I++)
		{
			if (RequiredShapeData[I] >= 0 && (int32)ShapeData[I] != RequiredShapeData[I])
			{
				UE_LOGF(LogNNEDenoiser, Error, "%ls does not have required shape (expected %d, got %d)!", *Label, RequiredShapeData[I], ShapeData[I])
				return false;
			}
		}

		return true;
	}

	template <typename PixelType>
	void CopyTextureFromGPUToCPU(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, FIntPoint Size, TArray<PixelType>& DstArray)
	{
		SCOPED_NAMED_EVENT_TEXT("NNEDenoiser.CopyTextureFromGPUToCPU", FColor::Magenta);

		FRHIGPUTextureReadback Readback(TEXT("NNEDenoiser.CopyTextureFromGPUToCPU"));
		Readback.EnqueueCopy(RHICmdList, SrcTexture, FIntVector::ZeroValue, 0, FIntVector(Size.X, Size.Y, 1));

		RHICmdList.SubmitAndBlockUntilGPUIdle();

		int32_t SrcStride = 0;
		const PixelType* SrcBuffer = static_cast<PixelType*>(Readback.Lock(SrcStride, nullptr));

		PixelType* DstBuffer = DstArray.GetData();
		for (int Y = 0; Y < Size.Y; Y++, DstBuffer += Size.X, SrcBuffer += SrcStride)
		{
			FPlatformMemory::Memcpy(DstBuffer, SrcBuffer, Size.X * sizeof(PixelType));

		}
		Readback.Unlock();
	}

	template <typename PixelType>
	void CopyTextureFromCPUToGPU(FRHICommandListImmediate& RHICmdList, const TArray<PixelType>& SrcArray, FIntPoint Size, FRHITexture* DstTexture)
	{
		SCOPED_NAMED_EVENT_TEXT("NNEDenoiser.CopyTextureFromCPUToGPU", FColor::Magenta);

		uint32 DestStride;
		PixelType* DstBuffer = static_cast<PixelType*>(RHICmdList.LockTexture2D(DstTexture, 0, RLM_WriteOnly, DestStride, false));
		DestStride /= sizeof(PixelType);
		const PixelType* SrcBuffer = SrcArray.GetData();
		for (int32 Y = 0; Y < Size.Y; Y++, SrcBuffer += Size.X, DstBuffer += DestStride)
		{
			FPlatformMemory::Memcpy(DstBuffer, SrcBuffer, Size.X * sizeof(PixelType));
		}
		RHICmdList.UnlockTexture2D(DstTexture, 0, false);
	}

	template <typename ElementType>
	void CopyBufferFromGPUToCPU(FRHICommandListImmediate& RHICmdList, FRHIBuffer* SrcBuffer, int32 Count, TArray<ElementType>& DstArray)
	{
		SCOPED_NAMED_EVENT_TEXT("NNEDenoiser.CopyBufferFromGPUToCPU", FColor::Magenta);

		const int32 NumBytes = Count * sizeof(ElementType);

		FRHIGPUBufferReadback Readback(TEXT("NNEDenoiser.CopyBufferFromGPUToCPU"));
		Readback.EnqueueCopy(RHICmdList, SrcBuffer, NumBytes);

		RHICmdList.SubmitAndBlockUntilGPUIdle();

		const ElementType* Src = static_cast<ElementType*>(Readback.Lock(NumBytes));
		ElementType* Dst = DstArray.GetData();
		
		FPlatformMemory::Memcpy(Dst, Src, NumBytes);

		Readback.Unlock();
	}

	template <typename ElementType>
	void CopyBufferFromCPUToGPU(FRHICommandListImmediate& RHICmdList, const TArray<ElementType>& SrcArray, int32 Count, FRHIBuffer* DstBuffer)
	{
		SCOPED_NAMED_EVENT_TEXT("NNEDenoiser.CopyBufferFromCPUToGPU", FColor::Magenta);

		ElementType* Dst = static_cast<ElementType*>(RHICmdList.LockBuffer(DstBuffer, 0, Count * sizeof(ElementType), RLM_WriteOnly));
		const ElementType* Src = SrcArray.GetData();
		
		FPlatformMemory::Memcpy(Dst, Src, Count * sizeof(ElementType));

		RHICmdList.UnlockBuffer(DstBuffer);
	}

	inline EPixelFormat GetBufferFormat(ENNETensorDataType TensorDataType)
	{
		switch (TensorDataType)
		{
			case ENNETensorDataType::Half: return EPixelFormat::PF_R16F;
			case ENNETensorDataType::Float: return EPixelFormat::PF_R32_FLOAT;
			default:
				unimplemented();
		}
		return EPixelFormat::PF_Unknown;
	}

	inline UE::NNEDenoiserShaders::Internal::EDataType GetDenoiserShaderDataType(ENNETensorDataType TensorDataType)
	{
		using UE::NNEDenoiserShaders::Internal::EDataType;

		switch (TensorDataType)
		{
			case ENNETensorDataType::Half: return EDataType::Half;
			case ENNETensorDataType::Float: return EDataType::Float;
			default:
				unimplemented();
		}
		return EDataType::None;
	}

	inline UE::NNEDenoiserShaders::Internal::EDataType GetDenoiserShaderDataType(EPixelFormat Format)
	{
		using UE::NNEDenoiserShaders::Internal::EDataType;

		switch (Format)
		{
			case EPixelFormat::PF_R16F:
			case EPixelFormat::PF_FloatRGBA: return EDataType::Half;

			case EPixelFormat::PF_R32_FLOAT:
			case EPixelFormat::PF_A32B32G32R32F: return EDataType::Float;
			
			default:
				unimplemented();
		}
		return EDataType::None;
	} 

}
