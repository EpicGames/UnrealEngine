// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserViewExtension.h"
#include "NNE.h"
#include "NNEDenoiserAsset.h"
#include "NNEDenoiserAutoExposure.h"
#include "NNEDenoiserGenericDenoiser.h"
#include "NNEDenoiserIOMappingData.h"
#include "NNEDenoiserIOProcessBase.h"
#include "NNEDenoiserLog.h"
#include "NNEDenoiserModelInstanceCPU.h"
#include "NNEDenoiserModelInstanceGPU.h"
#include "NNEDenoiserModelInstanceRDG.h"
#include "NNEDenoiserPathTracingDenoiser.h"
#include "NNEDenoiserPathTracingSpatialTemporalDenoiser.h"
#include "NNEDenoiserResourceMapping.h"
#include "NNEDenoiserTemporalAsset.h"
#include "NNEDenoiserTransferFunctionOidn.h"
#include "NNEDenoiserUtils.h"
#include "NNEModelData.h"
#include "PathTracingDenoiser.h"
#include "SceneView.h"
#include "RenderUtils.h"

namespace UE::NNEDenoiser::Private
{

constexpr int32 NO_SHOW_PATH_TRACING_TIMEOUT = 300; // Number of calls to BeginRenderViewFamily() wihtout Path Tracing show flag set in view family

static TAutoConsoleVariable<bool> CVarNNEDenoiser(
	TEXT("NNEDenoiser"),
	true,
	TEXT("Enable the NNE Denoiser.")
);

static TAutoConsoleVariable<int32> CVarNNEDenoiserAsset(
	TEXT("NNEDenoiser.Asset"),
	0,
	TEXT("Defines denoiser asset to used to create the denoiser.\n")
	TEXT("  0: Use denoiser asset defined by Project Settings\n")
	TEXT("  1: OIDN Fast\n")
	TEXT("  2: OIDN Balanced\n")
	TEXT("  3: OIDN High Quality\n")
	TEXT("  4: OIDN Fast | Alpha\n")
	TEXT("  5: OIDN Balanced | Alpha\n")
	TEXT("  6: OIDN High Quality | Alpha")
);

static TAutoConsoleVariable<int32> CVarNNEDenoiserTemporalAsset(
	TEXT("NNEDenoiser.TemporalAsset"),
	0,
	TEXT("Defines temporal denoiser asset to used to create the temporal denoiser.\n")
	TEXT("  0: Use temporal denoiser asset defined by Project Settings\n")
	TEXT("  1: OIDN Balanced\n")
	TEXT("  2: OIDN Balanced | Alpha")
);

static TAutoConsoleVariable<int32> CVarNNEDenoiserRuntimeType(
	TEXT("NNEDenoiser.Runtime.Type"),
	2,
	TEXT("Defines the runtime type to run the denoiser model.\n")
	TEXT("  0: CPU\n")
	TEXT("  1: GPU\n")
	TEXT("  2: RDG\n")
);

static TAutoConsoleVariable<FString> CVarNNEDenoiserRuntimeName(
	TEXT("NNEDenoiser.Runtime.Name"),
	FString(),
	TEXT("Defines the runtime name to run the denoiser model. Leave empty to use default.")
);

EDenoiserRuntimeType GetDenoiserRuntimeTypeFromCVar()
{
	const int32 Value = CVarNNEDenoiserRuntimeType.GetValueOnGameThread();
	const int32 Min = (int32)EDenoiserRuntimeType::CPU;
	const int32 Max = (int32)EDenoiserRuntimeType::RDG;
	return static_cast<EDenoiserRuntimeType>(FMath::Clamp(Value, Min, Max));
}

FString GetDenoiserAssetNameFromCVarAndSettings(const UNNEDenoiserSettings* Settings)
{
	const int32 Idx = FMath::Clamp(CVarNNEDenoiserAsset.GetValueOnGameThread(), 0, 6);
	switch(Idx)
	{
		case 0: return !Settings->DenoiserAsset.IsNull() ? Settings->DenoiserAsset.ToString() : FString();

		case 1: return TEXT("/NNEDenoiser/NNED_Oidn2-3_Fast.NNED_Oidn2-3_Fast");
		case 2: return TEXT("/NNEDenoiser/NNED_Oidn2-3_Balanced.NNED_Oidn2-3_Balanced");
		case 3: return TEXT("/NNEDenoiser/NNED_Oidn2-3_HighQuality.NNED_Oidn2-3_HighQuality");

		// Alpha
		case 4: return TEXT("/NNEDenoiser/NNED_Oidn2-3_Fast_Alpha.NNED_Oidn2-3_Fast_Alpha");
		case 5: return TEXT("/NNEDenoiser/NNED_Oidn2-3_Balanced_Alpha.NNED_Oidn2-3_Balanced_Alpha");
		case 6: return TEXT("/NNEDenoiser/NNED_Oidn2-3_HighQuality_Alpha.NNED_Oidn2-3_HighQuality_Alpha");
	}
	check(false);
	return FString();
}

FString GetDenoiserTemporalAssetNameFromCVarAndSettings(const UNNEDenoiserSettings* Settings)
{
	const int32 Idx = FMath::Clamp(CVarNNEDenoiserTemporalAsset.GetValueOnGameThread(), 0, 6);
	switch(Idx)
	{
		case 0: return !Settings->TemporalDenoiserAsset.IsNull() ? Settings->TemporalDenoiserAsset.ToString() : FString();

		case 1: return TEXT("/NNEDenoiser/NNEDT_Oidn2-3_Balanced.NNEDT_Oidn2-3_Balanced");

		// Alpha
		case 2: return TEXT("/NNEDenoiser/NNEDT_Oidn2-3_Balanced_Alpha.NNEDT_Oidn2-3_Balanced_Alpha");
	}
	check(false);
	return FString();
}

TUniquePtr<FGenericDenoiser> CreateNNEDenoiser(
	UNNEModelData& ModelData,
	EDenoiserRuntimeType RuntimeType,
	const FString& RuntimeNameOverride,
	TUniquePtr<IInputProcess> InputProcess,
	TUniquePtr<IOutputProcess> OutputProcess,
	FParameters Parameters,
	TUniquePtr<IAutoExposure> AutoExposure,
	TSharedPtr<ITransferFunction> TransferFunction)
{
	static TMap<EDenoiserRuntimeType, FString> DefaultRuntimeNames;
	DefaultRuntimeNames.Emplace(EDenoiserRuntimeType::CPU, TEXT("NNERuntimeORTCpu"));
	DefaultRuntimeNames.Emplace(EDenoiserRuntimeType::GPU, TEXT("NNERuntimeORTDml"));
	DefaultRuntimeNames.Emplace(EDenoiserRuntimeType::RDG, TEXT("NNERuntimeORTDml"));

	const static FString FallbackRuntimeNameRDG = TEXT("NNERuntimeRDGHlsl");

	TArray<TPair<EDenoiserRuntimeType, FString>> RuntimePriorityQueue;
	if (!RuntimeNameOverride.IsEmpty())
	{
		RuntimePriorityQueue.AddUnique({RuntimeType, RuntimeNameOverride});
	}
	
	const FString& DefaultRuntimeName = DefaultRuntimeNames.FindChecked(RuntimeType);
	if (RuntimeNameOverride != DefaultRuntimeName)
	{
		RuntimePriorityQueue.AddUnique({RuntimeType, DefaultRuntimeName});
	}

	if (RuntimeType >= EDenoiserRuntimeType::RDG)
	{
		if (RuntimeNameOverride != FallbackRuntimeNameRDG)
		{
			RuntimePriorityQueue.AddUnique({EDenoiserRuntimeType::RDG, FallbackRuntimeNameRDG});
		}

		const FString& DefaultRuntimeNameGPU = DefaultRuntimeNames.FindChecked(EDenoiserRuntimeType::GPU);
		RuntimePriorityQueue.AddUnique({EDenoiserRuntimeType::GPU, DefaultRuntimeNameGPU});
	}

	if (RuntimeType >= EDenoiserRuntimeType::GPU)
	{
		const FString& DefaultRuntimeNameCPU = DefaultRuntimeNames.FindChecked(EDenoiserRuntimeType::CPU);
		RuntimePriorityQueue.AddUnique({EDenoiserRuntimeType::CPU, DefaultRuntimeNameCPU});
	}

	TUniquePtr<IModelInstance> ModelInstance;
	for (const auto& Pair : RuntimePriorityQueue)
	{
		UE_LOGF(LogNNEDenoiser, Log, "Try create model instance with runtime %ls on %ls...", *Pair.Value, *UEnum::GetValueAsString(Pair.Key));

		if (Pair.Key == EDenoiserRuntimeType::CPU)
		{
			ModelInstance = FModelInstanceCPU::Make(ModelData, Pair.Value);
		}
		else if (Pair.Key == EDenoiserRuntimeType::GPU)
		{
			ModelInstance = FModelInstanceGPU::Make(ModelData, Pair.Value);
		}
		else if (Pair.Key == EDenoiserRuntimeType::RDG)
		{
			ModelInstance = FModelInstanceRDG::Make(ModelData, Pair.Value);
		}

		if (ModelInstance.IsValid())
		{
			UE_LOGF(LogNNEDenoiser, Display, "Created model instance with runtime %ls on %ls", *Pair.Value, *UEnum::GetValueAsString(Pair.Key));
			break;
		}
	}

	if (!ModelInstance.IsValid())
	{
		UE_LOGF(LogNNEDenoiser, Error, "Could not create denoiser!");
		return {};
	}

	return MakeUnique<FGenericDenoiser>(
		MoveTemp(ModelInstance),
		MoveTemp(InputProcess),
		MoveTemp(OutputProcess),
		MoveTemp(Parameters),
		MoveTemp(AutoExposure),
		TransferFunction);
}

template<class AssetType>
FParameters GetParametersValidated(const AssetType& DenoiserAsset)
{
	FParameters Parameters{
		.TilingConfig =
		{
			.Alignment = DenoiserAsset.TilingConfig.Alignment,
			.Overlap = DenoiserAsset.TilingConfig.Overlap,
			.MaxSize = DenoiserAsset.TilingConfig.MaxSize,
			.MinSize = DenoiserAsset.TilingConfig.MinSize
		}
	};

	if (Parameters.TilingConfig.Alignment < 1)
	{
		UE_LOGF(LogNNEDenoiser, Warning, "Tiling alignment should be at least 1!");
		Parameters.TilingConfig.Alignment = 1;
	}

	if (Parameters.TilingConfig.Overlap % Parameters.TilingConfig.Alignment != 0)
	{
		UE_LOGF(LogNNEDenoiser, Warning, "Tiling overlap should be aligned by %d!", Parameters.TilingConfig.Alignment);
		Parameters.TilingConfig.Overlap = RoundUp(Parameters.TilingConfig.Overlap, Parameters.TilingConfig.Alignment);
	}

	if (Parameters.TilingConfig.MinSize < Parameters.TilingConfig.Overlap + Parameters.TilingConfig.Alignment)
	{
		UE_LOGF(LogNNEDenoiser, Warning, "Minimum tile size should be at least overlap + alignment = %d!", Parameters.TilingConfig.Overlap + Parameters.TilingConfig.Alignment);
		Parameters.TilingConfig.MinSize = Parameters.TilingConfig.Overlap + Parameters.TilingConfig.Alignment;
	}

	if (Parameters.TilingConfig.MinSize % Parameters.TilingConfig.Alignment != 0)
	{
		UE_LOGF(LogNNEDenoiser, Warning, "Minimum tile size should be aligned by %d!", Parameters.TilingConfig.Alignment);
		Parameters.TilingConfig.MinSize = RoundUp(Parameters.TilingConfig.MinSize, Parameters.TilingConfig.Alignment);
	}

	if (Parameters.TilingConfig.MaxSize > 0 && Parameters.TilingConfig.MaxSize < Parameters.TilingConfig.MinSize)
	{
		UE_LOGF(LogNNEDenoiser, Warning, "Maximum tile size should be at least minimum tile size!");
		Parameters.TilingConfig.MaxSize = Parameters.TilingConfig.MinSize;
	}

	if (Parameters.TilingConfig.MaxSize % Parameters.TilingConfig.Alignment != 0)
	{
		UE_LOGF(LogNNEDenoiser, Warning, "Maximum tile size should be aligned by %d!", Parameters.TilingConfig.Alignment);
		Parameters.TilingConfig.MaxSize = RoundUp(Parameters.TilingConfig.MaxSize, Parameters.TilingConfig.Alignment);
	}

	return Parameters;
}

template<class AssetType, class InputMappingType, class OutputMappingType>
TUniquePtr<FGenericDenoiser> CreateNNEDenoiserFromAsset(const FString& AssetName, EDenoiserRuntimeType RuntimeType, const FString& RuntimeNameOverride, int32 MaximumTileSizeOverride)
{
	if (AssetName.IsEmpty())
	{
		UE_LOGF(LogNNEDenoiser, Error, "Asset name not set!");
		return {};
	}

	AssetType *DenoiserAsset = LoadObject<AssetType>(nullptr, *AssetName);
	if (!DenoiserAsset)
	{
		UE_LOGF(LogNNEDenoiser, Error, "Could not load denoiser model data asset!");
		return {};
	}

	UNNEModelData* ModelData = DenoiserAsset->ModelData.LoadSynchronous();
	if (!ModelData)
	{
		UE_LOGF(LogNNEDenoiser, Error, "Asset does not contain model data!");
		return {};
	}
	UDataTable* InputMappingTable = DenoiserAsset->InputMapping.LoadSynchronous();
	if (InputMappingTable)
	{
		UE_LOGF(LogNNEDenoiser, Log, "Loaded input mapping from %ls", *DenoiserAsset->InputMapping.GetAssetName());
	}
	UDataTable* OutputMappingTable = DenoiserAsset->OutputMapping.LoadSynchronous();
	if (OutputMappingTable)
	{
		UE_LOGF(LogNNEDenoiser, Log, "Loaded output mapping from %ls", *DenoiserAsset->OutputMapping.GetAssetName());
	}

	FResourceMappingList InputLayout;
	if (InputMappingTable)
	{
		InputLayout = MakeTensorLayout<InputMappingType>(InputMappingTable);
	}

	FResourceMappingList OutputLayout;
	if (OutputMappingTable)
	{
		OutputLayout = MakeTensorLayout<OutputMappingType>(OutputMappingTable);
	}

	TUniquePtr<FAutoExposure> AutoExposure;
	TSharedPtr<ITransferFunction> TransferFunction;
	if (AssetName.Contains(TEXT("oidn2")))
	{
		AutoExposure = MakeUnique<FAutoExposure>();
		TransferFunction = MakeShared<Oidn::FTransferFunction>();
	}

	TUniquePtr<IInputProcess> InputProcess = MakeUnique<FInputProcessBase>(MoveTemp(InputLayout), TransferFunction);
	TUniquePtr<IOutputProcess> OutputProcess = MakeUnique<FOutputProcessBase>(MoveTemp(OutputLayout), TransferFunction);

	FParameters Parameters = GetParametersValidated(*DenoiserAsset);

	if (MaximumTileSizeOverride > -1)
	{
		if (MaximumTileSizeOverride < Parameters.TilingConfig.MaxSize)
		{
			if (MaximumTileSizeOverride < Parameters.TilingConfig.MinSize)
			{
				UE_LOGF(LogNNEDenoiser, Display, "Maximum tile size override %d smaller than minimum tile size %d supported by asset!", MaximumTileSizeOverride, Parameters.TilingConfig.MinSize);
			}

			int32 NewMaxSize = FMath::Clamp(MaximumTileSizeOverride, Parameters.TilingConfig.MinSize, Parameters.TilingConfig.MaxSize);
			if (NewMaxSize < Parameters.TilingConfig.MaxSize)
			{
				UE_LOGF(LogNNEDenoiser, Log, "Applied maximum tile size override %d to assets maximum tile size %d.", NewMaxSize, Parameters.TilingConfig.MaxSize);

				Parameters.TilingConfig.MaxSize = NewMaxSize;
			}
		}
	}

	return CreateNNEDenoiser(
		*ModelData,
		RuntimeType,
		RuntimeNameOverride,
		MoveTemp(InputProcess),
		MoveTemp(OutputProcess),
		MoveTemp(Parameters),
		MoveTemp(AutoExposure),
		TransferFunction);
}

FViewExtension::FViewExtension(const FAutoRegister& AutoRegister) : FSceneViewExtensionBase(AutoRegister)
{

}

FViewExtension::~FViewExtension()
{
#if WITH_EDITOR
	if(UObjectInitialized())
	{
		GetMutableDefault<UNNEDenoiserSettings>()->OnSettingChanged().RemoveAll(this);
	}
#endif

	UnregisterDenoiser(TEXT("NNEDenoiser"));
}

void FViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	TimeoutCounter = !!InViewFamily.EngineShowFlags.PathTracing ? NO_SHOW_PATH_TRACING_TIMEOUT : TimeoutCounter > 0 ? TimeoutCounter - 1 : 0;

	const bool bIsActiveBefore = bIsActive;
	bIsActive = IsRayTracingEnabled() && CVarNNEDenoiser.GetValueOnGameThread() && TimeoutCounter > 0;
	if (bIsActive != bIsActiveBefore)
	{
		if (bIsActive)
		{
			UE_LOGF(LogNNEDenoiser, Log, "Activating...");
		}
		else
		{
			UE_LOGF(LogNNEDenoiser, Log, "Deactivating and free all resources...");
		}
	}

	if (!bIsActive)
	{
		return;
	}

	// Check for changes
	const UNNEDenoiserSettings* Settings = GetDefault<UNNEDenoiserSettings>();
	bool bNeedsUpdate = bIsActive != bIsActiveBefore;
	bNeedsUpdate |= GetDenoiserRuntimeTypeFromCVar() != RuntimeType;
	bNeedsUpdate |= CVarNNEDenoiserRuntimeName.GetValueOnGameThread() != RuntimeName;
	bNeedsUpdate |= GetDenoiserAssetNameFromCVarAndSettings(Settings) != AssetName;
	bNeedsUpdate |= GetDenoiserTemporalAssetNameFromCVarAndSettings(Settings) != TemporalAssetName;
	bNeedsUpdate |= Settings->MaximumTileSizeOverride != MaximumTileSizeOverride;

	if (!bNeedsUpdate)
	{
		return;
	}

	RuntimeType = GetDenoiserRuntimeTypeFromCVar();
	RuntimeName = CVarNNEDenoiserRuntimeName.GetValueOnGameThread();
	AssetName = GetDenoiserAssetNameFromCVarAndSettings(Settings);
	TemporalAssetName = GetDenoiserTemporalAssetNameFromCVarAndSettings(Settings);
	MaximumTileSizeOverride = Settings->MaximumTileSizeOverride;

	TUniquePtr<FGenericDenoiser> Denoiser = CreateNNEDenoiserFromAsset<UNNEDenoiserAsset, FNNEDenoiserInputMappingData, FNNEDenoiserOutputMappingData>(AssetName, RuntimeType, RuntimeName, MaximumTileSizeOverride);
	if (Denoiser.IsValid())
	{
		UE_LOGF(LogNNEDenoiser, Log, "Create denoiser from asset %ls...", *AssetName);

		DenoiserToSwap = MakeUnique<FPathTracingDenoiser>(MoveTemp(Denoiser));
	}
	else
	{
		UE_LOGF(LogNNEDenoiser, Error, "Could not create denoiser from asset %ls!", *AssetName);
	}

	TUniquePtr<FGenericDenoiser> TemporalDenoiser = CreateNNEDenoiserFromAsset<UNNEDenoiserTemporalAsset, FNNEDenoiserTemporalInputMappingData, FNNEDenoiserTemporalOutputMappingData>(TemporalAssetName, RuntimeType, RuntimeName, MaximumTileSizeOverride);
	if (TemporalDenoiser.IsValid())
	{
		UE_LOGF(LogNNEDenoiser, Log, "Create temporal denoiser from asset %ls...", *TemporalAssetName);

		SpatialTemporalDenoiserToSwap = MakeUnique<FPathTracingSpatialTemporalDenoiser>(MoveTemp(TemporalDenoiser));
	}
	else
	{
		UE_LOGF(LogNNEDenoiser, Error, "Could not create temporal denoiser from asset %ls!", *TemporalAssetName);
	}
}

void FViewExtension::PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	if (!bIsActive)
	{
		UnregisterDenoiser(TEXT("NNEDenoiser"));
	}
	else
	{
		if (DenoiserToSwap.IsValid() || SpatialTemporalDenoiserToSwap.IsValid())
		{
			UnregisterDenoiser(TEXT("NNEDenoiser"));
		}

		if (DenoiserToSwap.IsValid())
		{
			RegisterSpatialDenoiser(MoveTemp(DenoiserToSwap),TEXT("NNEDenoiser"));
		}

		if (SpatialTemporalDenoiserToSwap.IsValid())
		{
			RegisterSpatialTemporalDenoiser(MoveTemp(SpatialTemporalDenoiserToSwap), TEXT("NNEDenoiser"));
		}
	}
}

} // namespace UE::NNEDenoiser::Private
