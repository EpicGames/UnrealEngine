// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"
#include "NNEDenoiserSettings.h"

namespace UE::Renderer::Private
{
	class IPathTracingDenoiser;
	class IPathTracingSpatialTemporalDenoiser;
}

namespace UE::NNEDenoiser::Private
{

class FViewExtension final : public FSceneViewExtensionBase
{
	public:
		FViewExtension(const FAutoRegister& AutoRegister);
		virtual ~FViewExtension();

		void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;

		void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;

	private:
		TUniquePtr<UE::Renderer::Private::IPathTracingDenoiser> DenoiserToSwap;
		TUniquePtr<UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser> SpatialTemporalDenoiserToSwap;

		// Cached settings and CVar values
		bool bIsActive = false;
		int32 TimeoutCounter = 0;
		EDenoiserRuntimeType RuntimeType;
		FString RuntimeName;
		FString AssetName;
		FString TemporalAssetName;
		uint32 MaximumTileSizeOverride;
	};

} // namespace UE::NNEDenoiser::Private