// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "Engine/DataTable.h"
#include "NNEDenoiserTilingConfig.h"
#include "NNEModelData.h"

#include "NNEDenoiserAsset.generated.h"

/** Denoiser model data asset */
UCLASS(MinimalAPI, BlueprintType)
class UNNEDenoiserAsset : public UDataAsset
{
	GENERATED_BODY()

public:
	/** NNE model data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	TSoftObjectPtr<UNNEModelData> ModelData;

	/** Input mapping table */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser, meta = (RequiredAssetDataTags = "RowStructure=/Script/NNEDenoiser.NNEDenoiserInputMappingData"))
	TSoftObjectPtr<UDataTable> InputMapping;

	/** Output mapping table */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser, meta = (RequiredAssetDataTags = "RowStructure=/Script/NNEDenoiser.NNEDenoiserOutputMappingData"))
	TSoftObjectPtr<UDataTable> OutputMapping;

	/** Tiling configuration */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	FTilingConfig TilingConfig{};
};
