// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "NNEDenoiserResourceName.h"

#include "NNEDenoiserIOMappingData.generated.h"

/** An enum to represent resource names used for denoiser input mapping */
UENUM()
enum class EInputResourceName : uint8
{
	Color,
	Albedo,
	Normal,
	Output
};

/** An enum to represent resource names used for denoiser output mapping */
UENUM()
enum class EOutputResourceName : uint8
{
	Output
};

/** An enum to represent resource names used for temporal denoiser input mapping */
UENUM()
enum class ETemporalInputResourceName : uint8
{
	Color,
	Albedo,
	Normal,
	Flow,
	Output
};

/** An enum to represent resource names used for temporal denoiser output mapping */
UENUM()
enum class ETemporalOutputResourceName : uint8
{
	Output
};

/** Table row base for denoiser basic input and output mapping */
USTRUCT(BlueprintType)
struct FNNEDenoiserBaseMappingData : public FTableRowBase
{
	GENERATED_USTRUCT_BODY()

public:
	/** Input/output tensor index */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	int32 TensorIndex = 0;

	/** Input/output tensor channel */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	int32 TensorChannel = 0;

	/** Resource channel */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	int32 ResourceChannel = 0;
};

/** Table row base for denoiser input mapping */
USTRUCT(BlueprintType)
struct FNNEDenoiserInputMappingData : public FNNEDenoiserBaseMappingData
{
	GENERATED_USTRUCT_BODY()

	/** Mapped resource name */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	EInputResourceName Resource = EInputResourceName::Color;
};

/** Table row base for denoiser output mapping */
USTRUCT(BlueprintType)
struct FNNEDenoiserOutputMappingData : public FNNEDenoiserBaseMappingData
{
	GENERATED_USTRUCT_BODY()

	/** Mapped resource name */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	EOutputResourceName Resource = EOutputResourceName::Output;
};

/** Table row base for temporal denoiser input mapping */
USTRUCT(BlueprintType)
struct FNNEDenoiserTemporalInputMappingData : public FNNEDenoiserBaseMappingData
{
	GENERATED_USTRUCT_BODY()

public:
	/** Mapped resource name */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	ETemporalInputResourceName Resource = ETemporalInputResourceName::Color;

	/** Resource frame index */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	int32 FrameIndex = 0;
};

/** Table row base for temporal denoiser output mapping */
USTRUCT(BlueprintType)
struct FNNEDenoiserTemporalOutputMappingData : public FNNEDenoiserBaseMappingData
{
	GENERATED_USTRUCT_BODY()

	/** Mapped resource name */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	ETemporalOutputResourceName Resource = ETemporalOutputResourceName::Output;
};

namespace UE::NNEDenoiser
{

EResourceName ToResourceName(EInputResourceName Name);
EResourceName ToResourceName(EOutputResourceName Name);
EResourceName ToResourceName(ETemporalInputResourceName Name);
EResourceName ToResourceName(ETemporalOutputResourceName Name);

} // namespace UE::NNEDenoiser