// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEDenoiserResourceName.generated.h"

/** An enum to represent resource names used for input and output mapping */
UENUM()
enum class EResourceName : uint8
{
	Color,
	Albedo,
	Normal,
	Flow,
	Output
};