// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettingsBackedByCVars.h"
#include "NNEDenoiserAsset.h"
#include "NNEDenoiserTemporalAsset.h"

#include "NNEDenoiserSettings.generated.h"

#define UE_API NNEDENOISER_API

/** An enum to represent denoiser NNE runtime type */
UENUM()
enum EDenoiserRuntimeType : uint8
{
	CPU,
	GPU,
	RDG
};

/** Settings to select a NNE Denoiser and its runtime */
UCLASS(MinimalAPI, Config = Engine, meta = (DisplayName = "NNE Denoiser"))
class UNNEDenoiserSettings : public UDeveloperSettingsBackedByCVars
{
	GENERATED_BODY()

public:
	UE_API UNNEDenoiserSettings();

	UE_API virtual void PostInitProperties() override;

	/** Denoiser asset data used to create a NNE Denoiser */
	UPROPERTY(Config, EditAnywhere, Category = "NNE Denoiser", meta = (DisplayName = "Denoiser Asset", ToolTip = "Select the denoiser asset"))
	TSoftObjectPtr<UNNEDenoiserAsset> DenoiserAsset;

	/** Temporal denoiser asset data used to create a NNE Denoiser (Currently not used and therefore "hidden") */
	TSoftObjectPtr<UNNEDenoiserTemporalAsset> TemporalDenoiserAsset;

	/** Override the maximum tile size defined per asset, but be aware not to set it lower than the assets minimum tile size (This can reduce GPU memory usage for GPU and RDG backed denoisers). */
	UPROPERTY(Config, EditAnywhere, Category = "NNE Denoiser", meta = (DisplayName = "Maximum tile size override", ToolTip = "Override the maximum tile size given by the asset.\nSpecial values:\n   -1 = Do not override"))
	int32 MaximumTileSizeOverride = -1;

private:
	/** Runtime type used to run the NNE Denoiser model. Backed by the console variable 'NNEDenoiser.Runtime.Type'. */
	UPROPERTY(Config, EditAnywhere, Category = "NNE Denoiser", meta = (DisplayName = "Runtime Type", ToolTip = "Select a Runtime type", ConsoleVariable = "NNEDenoiser.Runtime.Type"))
	TEnumAsByte<EDenoiserRuntimeType> RuntimeType;

	/** Runtime name used to run the NNE Denoiser model. Backed by the console variable 'NNEDenoiser.Runtime.Name'. */
	UPROPERTY(Config, EditAnywhere, Category = "NNE Denoiser", meta = (DisplayName = "Runtime Name Override", ToolTip = "(Optional) Specify the Runtime name", ConsoleVariable = "NNEDenoiser.Runtime.Name"))
	FString RuntimeName;
};

#undef UE_API
