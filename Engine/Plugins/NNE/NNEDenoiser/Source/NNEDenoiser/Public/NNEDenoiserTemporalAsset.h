// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "Engine/DataTable.h"
#include "NNEDenoiserTilingConfig.h"
#include "NNEModelData.h"

#include "NNEDenoiserTemporalAsset.generated.h"

/** Denoiser model data asset */
UCLASS(MinimalAPI, BlueprintType)
class UNNEDenoiserTemporalAsset : public UDataAsset
{
	GENERATED_BODY()

public:
	/** NNE model data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	TSoftObjectPtr<UNNEModelData> ModelData;

	/** Input mapping table */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser, meta = (RequiredAssetDataTags = "RowStructure=/Script/NNEDenoiser.NNEDenoiserTemporalInputMappingData"))
	TSoftObjectPtr<UDataTable> InputMapping;

	/** Output mapping table */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser, meta = (RequiredAssetDataTags = "RowStructure=/Script/NNEDenoiser.NNEDenoiserTemporalOutputMappingData"))
	TSoftObjectPtr<UDataTable> OutputMapping;

	/** Tiling configuration */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	FTilingConfig TilingConfig{};
};
