// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEDenoiserTilingConfig.generated.h"

/** Tiling configuration for fixed and dynamic size models */
USTRUCT(BlueprintType)
struct FTilingConfig
{
	GENERATED_BODY()

	/** Tile size alignment (applies only to dynamic size models) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser, meta = (DisplayName = "Size Alignment"))
	int32 Alignment = 1;

	/** Tile overlap */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	int32 Overlap = 0;

	/** Maximum tile size (applies only to dynamic size models) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	int32 MaxSize = 0;

	/** Minimum tile size (applies only to dynamic size models) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NNEDenoiser)
	int32 MinSize = 1;
};