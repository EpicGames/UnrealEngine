// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

#define UE_API NNEDENOISERSHADERS_API

namespace UE::NNEDenoiserShaders::Internal
{
	
	class FAutoExposureDownsampleConstants
	{
	public:
		static constexpr int32 MAX_BIN_SIZE{16};
		static constexpr int32 THREAD_GROUP_SIZE{MAX_BIN_SIZE};
	};

	class FAutoExposureDownsampleCS : public FGlobalShader
	{
		DECLARE_EXPORTED_GLOBAL_SHADER(FAutoExposureDownsampleCS, UE_API);
		SHADER_USE_PARAMETER_STRUCT(FAutoExposureDownsampleCS, FGlobalShader)

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, InputTextureWidth)
			SHADER_PARAMETER(int32, InputTextureHeight)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER(int32, NumBinsW)
			SHADER_PARAMETER(int32, NumBinsH)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutputBins)
		END_SHADER_PARAMETER_STRUCT()

		static UE_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static UE_API bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

	class FAutoExposureReduceConstants
	{
	public:
		static constexpr int32 THREAD_GROUP_SIZE{32};
		static constexpr float EPS{1e-8f};
		static constexpr float KEY{0.18f};
	};

	class FAutoExposureReduceCS : public FGlobalShader
	{
		DECLARE_EXPORTED_GLOBAL_SHADER(FAutoExposureReduceCS, UE_API);
		SHADER_USE_PARAMETER_STRUCT(FAutoExposureReduceCS, FGlobalShader)

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, InputSize)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InputBins)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutputSums)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutputCounts)
			SHADER_PARAMETER(int32, NumThreads)
		END_SHADER_PARAMETER_STRUCT()

		static UE_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static UE_API bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

	class FAutoExposureReduceFinalCS : public FGlobalShader
	{
		DECLARE_EXPORTED_GLOBAL_SHADER(FAutoExposureReduceFinalCS, UE_API);
		SHADER_USE_PARAMETER_STRUCT(FAutoExposureReduceFinalCS, FGlobalShader)

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, InputSize)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InputSums)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InputCounts)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutputBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static UE_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static UE_API bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

} // namespace UE::NNEDenoiser::Private

#undef UE_API
