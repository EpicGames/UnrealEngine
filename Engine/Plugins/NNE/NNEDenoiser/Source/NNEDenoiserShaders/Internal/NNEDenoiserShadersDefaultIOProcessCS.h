// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

#define UE_API NNEDENOISERSHADERS_API

namespace UE::NNEDenoiserShaders::Internal
{

	enum class EDefaultIOProcessInputKind : uint8
	{
		Color = 0,
		Albedo,
		Normal,
		Flow,
		Output,
		MAX
	};

	class FDefaultIOProcessConstants
	{
	public:
		static constexpr int32 THREAD_GROUP_SIZE{16};
	};
	
	class FDefaultIOProcessCS : public FGlobalShader
	{
		DECLARE_EXPORTED_GLOBAL_SHADER(FDefaultIOProcessCS, UE_API);
		SHADER_USE_PARAMETER_STRUCT(FDefaultIOProcessCS, FGlobalShader)

		class FDefaultIOProcessInputKind : SHADER_PERMUTATION_ENUM_CLASS("INPUT_KIND_INDEX", EDefaultIOProcessInputKind);
		using FPermutationDomain = TShaderPermutationDomain<FDefaultIOProcessInputKind>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, Width)
			SHADER_PARAMETER(int32, Height)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		END_SHADER_PARAMETER_STRUCT()

		static UE_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static UE_API bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

} // namespace UE::NNEDenoiser::Private

#undef UE_API
