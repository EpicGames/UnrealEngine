// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

#define UE_API NNEDENOISERSHADERS_API

namespace UE::NNEDenoiserShaders::Internal
{

	// Maps to ENNETensorDataType
	enum class EDataType : uint8
	{
		None = 0,
		// Char,
		// Boolean,
		Half = 3,
		Float,
		MAX
	};

	class FMappedCopyConstants
	{
	public:
		static constexpr int32 THREAD_GROUP_SIZE{16};
		static constexpr int32 MAX_NUM_MAPPED_CHANNELS{4};
	};

	// Note: The shader also supports Texture/Texture and Buffer/Buffer mapped copy, just add the Shader class.
	
	class FTextureBufferMappedCopyCS : public FGlobalShader
	{
		DECLARE_EXPORTED_GLOBAL_SHADER(FTextureBufferMappedCopyCS, UE_API);
		SHADER_USE_PARAMETER_STRUCT(FTextureBufferMappedCopyCS, FGlobalShader)

		class FInputDataType : SHADER_PERMUTATION_ENUM_CLASS("INPUT_DATA_TYPE_INDEX", EDataType);
		class FOutputDataType : SHADER_PERMUTATION_ENUM_CLASS("OUTPUT_DATA_TYPE_INDEX", EDataType);
		class FNumMappedChannels : SHADER_PERMUTATION_RANGE_INT("NUM_MAPPED_CHANNELS", 0, FMappedCopyConstants::MAX_NUM_MAPPED_CHANNELS + 1);
		using FPermutationDomain = TShaderPermutationDomain<FInputDataType, FOutputDataType, FNumMappedChannels>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, Width)
			SHADER_PARAMETER(int32, Height)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutputBuffer)
			SHADER_PARAMETER_ARRAY(FIntVector4, OutputChannel_InputChannel_Unused_Unused, [FMappedCopyConstants::MAX_NUM_MAPPED_CHANNELS])
		END_SHADER_PARAMETER_STRUCT()

		static UE_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static UE_API bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

	class FBufferTextureMappedCopyCS : public FGlobalShader
	{
		DECLARE_EXPORTED_GLOBAL_SHADER(FBufferTextureMappedCopyCS, UE_API);
		SHADER_USE_PARAMETER_STRUCT(FBufferTextureMappedCopyCS, FGlobalShader)

		class FInputDataType : SHADER_PERMUTATION_ENUM_CLASS("INPUT_DATA_TYPE_INDEX", EDataType);
		class FOutputDataType : SHADER_PERMUTATION_ENUM_CLASS("OUTPUT_DATA_TYPE_INDEX", EDataType);
		class FNumMappedChannels : SHADER_PERMUTATION_RANGE_INT("NUM_MAPPED_CHANNELS", 0, FMappedCopyConstants::MAX_NUM_MAPPED_CHANNELS + 1);
		using FPermutationDomain = TShaderPermutationDomain<FInputDataType, FOutputDataType, FNumMappedChannels>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, Width)
			SHADER_PARAMETER(int32, Height)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InputBuffer)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
			SHADER_PARAMETER_ARRAY(FIntVector4, OutputChannel_InputChannel_Unused_Unused, [FMappedCopyConstants::MAX_NUM_MAPPED_CHANNELS])
		END_SHADER_PARAMETER_STRUCT()

		static UE_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static UE_API bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

} // namespace UE::NNEDenoiser::Private

#undef UE_API
