// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

#define UE_API NNEDENOISERSHADERS_API

namespace UE::NNEDenoiserShaders::Internal
{

	enum class ETransferFunctionOidnMode : uint8
	{
		Forwward = 0,
		Inverse,
		MAX
	};

	class FTransferFunctionOidnConstants
	{
	public:
		static constexpr int32 THREAD_GROUP_SIZE{16};
	};
	
	class FTransferFunctionOidnCS : public FGlobalShader
	{
		DECLARE_EXPORTED_GLOBAL_SHADER(FTransferFunctionOidnCS, UE_API);
		SHADER_USE_PARAMETER_STRUCT(FTransferFunctionOidnCS, FGlobalShader)

		class FTransferFunctionOidnMode : SHADER_PERMUTATION_ENUM_CLASS("MODE", ETransferFunctionOidnMode);
		using FPermutationDomain = TShaderPermutationDomain<FTransferFunctionOidnMode>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, Width)
			SHADER_PARAMETER(int32, Height)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InputScaleBuffer)
			SHADER_PARAMETER(float, NormScale)
			SHADER_PARAMETER(float, InvNormScale)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		END_SHADER_PARAMETER_STRUCT()

		static UE_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static UE_API bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

} // namespace UE::NNEDenoiser::Private

#undef UE_API
