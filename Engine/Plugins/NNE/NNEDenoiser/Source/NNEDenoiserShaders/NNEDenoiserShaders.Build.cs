// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNEDenoiserShaders : ModuleRules
{
	public NNEDenoiserShaders(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"RenderCore"
			});
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
				"Renderer",
				"RHI"
			});
	}
}
