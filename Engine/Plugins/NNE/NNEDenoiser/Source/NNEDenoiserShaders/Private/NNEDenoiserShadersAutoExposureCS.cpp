// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserShadersAutoExposureCS.h"
#include "RenderUtils.h"

namespace UE::NNEDenoiserShaders::Internal
{

	void FAutoExposureDownsampleCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), FAutoExposureDownsampleConstants::THREAD_GROUP_SIZE);
		OutEnvironment.SetDefine(TEXT("MAX_FLT"), MAX_FLT);
	}

	void FAutoExposureReduceCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), FAutoExposureReduceConstants::THREAD_GROUP_SIZE);
		OutEnvironment.SetDefine(TEXT("EPS"), FAutoExposureReduceConstants::EPS);
	}

	void FAutoExposureReduceFinalCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), FAutoExposureReduceConstants::THREAD_GROUP_SIZE);
		OutEnvironment.SetDefine(TEXT("KEY"), FAutoExposureReduceConstants::KEY);
	}

	bool FAutoExposureDownsampleCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	bool FAutoExposureReduceCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	bool FAutoExposureReduceFinalCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	IMPLEMENT_GLOBAL_SHADER(FAutoExposureDownsampleCS, "/NNEDenoiserShaders/NNEDenoiserShadersAutoExposureDownsample.usf", "Downsample", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FAutoExposureReduceCS, "/NNEDenoiserShaders/NNEDenoiserShadersAutoExposureReduce.usf", "Reduce", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FAutoExposureReduceFinalCS, "/NNEDenoiserShaders/NNEDenoiserShadersAutoExposureReduceFinal.usf", "ReduceFinal", SF_Compute);

} // UE::NNEDenoiser::Private