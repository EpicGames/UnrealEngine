// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserShadersDefaultIOProcessCS.h"
#include "RenderUtils.h"

namespace UE::NNEDenoiserShaders::Internal
{
	void FDefaultIOProcessCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), FDefaultIOProcessConstants::THREAD_GROUP_SIZE);
		OutEnvironment.SetDefine(TEXT("MAX_FLT"), MAX_FLT);
	}

	bool FDefaultIOProcessCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	IMPLEMENT_GLOBAL_SHADER(FDefaultIOProcessCS, "/NNEDenoiserShaders/NNEDenoiserShadersDefaultIOProcess.usf", "IOProcess", SF_Compute);

} // UE::NNEDenoiser::Private