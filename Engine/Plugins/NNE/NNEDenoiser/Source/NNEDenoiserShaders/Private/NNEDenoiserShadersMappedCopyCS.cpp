// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEDenoiserShadersMappedCopyCS.h"
#include "RenderUtils.h"

namespace UE::NNEDenoiserShaders::Internal
{
	void CommonModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), FMappedCopyConstants::THREAD_GROUP_SIZE);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_MAPPED_CHANNELS"), FMappedCopyConstants::MAX_NUM_MAPPED_CHANNELS);
	}

	template<class GlobalShaderType>
	bool CommonShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!ShouldCompileRayTracingShadersForProject(Parameters.Platform))
		{
			return false;
		}

		typename GlobalShaderType::FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.template Get<typename GlobalShaderType::FInputDataType>() != EDataType::Half &&
			PermutationVector.template Get<typename GlobalShaderType::FInputDataType>() != EDataType::Float)
		{
			return false;
		}
		if (PermutationVector.template Get<typename GlobalShaderType::FOutputDataType>() != EDataType::Half &&
			PermutationVector.template Get<typename GlobalShaderType::FOutputDataType>() != EDataType::Float)
		{
			return false;
		}

		return true;
	}

	void FTextureBufferMappedCopyCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		CommonModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INTRINSIC_INPUT_TYPE"), 0);
		OutEnvironment.SetDefine(TEXT("INTRINSIC_OUTPUT_TYPE"), 1);
	}

	void FBufferTextureMappedCopyCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		CommonModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INTRINSIC_INPUT_TYPE"), 1);
		OutEnvironment.SetDefine(TEXT("INTRINSIC_OUTPUT_TYPE"), 0);
	}

	bool FTextureBufferMappedCopyCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return CommonShouldCompilePermutation<FTextureBufferMappedCopyCS>(Parameters);
	}

	bool FBufferTextureMappedCopyCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return CommonShouldCompilePermutation<FBufferTextureMappedCopyCS>(Parameters);
	}

	IMPLEMENT_GLOBAL_SHADER(FTextureBufferMappedCopyCS, "/NNEDenoiserShaders/NNEDenoiserShadersMappedCopy.usf", "MappedCopy", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FBufferTextureMappedCopyCS, "/NNEDenoiserShaders/NNEDenoiserShadersMappedCopy.usf", "MappedCopy", SF_Compute);

} // UE::NNEDenoiser::Private