// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"
#include "NNEDenoiserShadersLog.h"

DEFINE_LOG_CATEGORY(LogNNEDenoiserShaders);

class FNNEDenoiserShadersModule : public IModuleInterface
{
public:
	// Begin IModuleInterface
	virtual void StartupModule() override
	{
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("NNEDenoiser"));
		if (Plugin.IsValid())
		{
			const FString ShadersDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Shaders"));

			AddShaderSourceDirectoryMapping(TEXT("/NNEDenoiserShaders"), ShadersDir);
		}
		else
		{
			UE_LOGF(LogNNEDenoiserShaders, Warning, "Shaders directory not added. Failed to find NNEDenoiser plugin");
		}
	}

	virtual void ShutdownModule() override
	{

	}
};

IMPLEMENT_MODULE(FNNEDenoiserShadersModule, NNEDenoiserShaders);
